// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class EditorStyle : ModuleRules
{
	public EditorStyle(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		if (!Target.bCompileAgainstEditor)
		{
			throw new BuildException("Unable to instantiate EditorStyle module for non-editor targets.");
		}

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"SlateCore",
				"UnrealEd",
				"EditorFramework"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"Settings",
				"PropertyEditor"
			}
		);

		// DesktopPlatform is only available for Editor and Program targets (running on a desktop platform)
		bool IsDesktopPlatformType = Target.Platform == UnrealBuildTool.UnrealTargetPlatform.Win64
			|| Target.Platform == UnrealBuildTool.UnrealTargetPlatform.Mac
			|| Target.Platform == UnrealBuildTool.UnrealTargetPlatform.Linux
			|| Target.Platform == UnrealBuildTool.UnrealTargetPlatform.LinuxArm64;
		if (Target.bCompileAgainstEditor || (Target.Type == TargetType.Program && IsDesktopPlatformType))
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DesktopPlatform",
				}
			);
		}
	}
}
