// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorFontGlyphs.h"

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"

FText FEditorFontGlyphs::Angle_Double_Left(FText::FromString(FString(TEXT("\xf100"))));
FText FEditorFontGlyphs::Angle_Double_Right(FText::FromString(FString(TEXT("\xf101"))));
FText FEditorFontGlyphs::Angle_Left(FText::FromString(FString(TEXT("\xf104"))));
FText FEditorFontGlyphs::Angle_Right(FText::FromString(FString(TEXT("\xf105"))));
FText FEditorFontGlyphs::Angle_Up(FText::FromString(FString(TEXT("\xf106"))));
FText FEditorFontGlyphs::Archive(FText::FromString(FString(TEXT("\xf187"))));
FText FEditorFontGlyphs::Arrow_Circle_O_Right(FText::FromString(FString(TEXT("\xf18e"))));
FText FEditorFontGlyphs::Arrow_Left(FText::FromString(FString(TEXT("\xf060"))));
FText FEditorFontGlyphs::Arrow_Right(FText::FromString(FString(TEXT("\xf061"))));
FText FEditorFontGlyphs::Arrows(FText::FromString(FString(TEXT("\xf047"))));
FText FEditorFontGlyphs::Ban(FText::FromString(FString(TEXT("\xf05e"))));
FText FEditorFontGlyphs::Bars(FText::FromString(FString(TEXT("\xf0c9"))));
FText FEditorFontGlyphs::Book(FText::FromString(FString(TEXT("\xf02d"))));
FText FEditorFontGlyphs::Bug(FText::FromString(FString(TEXT("\xf188"))));
FText FEditorFontGlyphs::Car(FText::FromString(FString(TEXT("\xf1b9"))));
FText FEditorFontGlyphs::Caret_Down(FText::FromString(FString(TEXT("\xf0d7"))));
FText FEditorFontGlyphs::Caret_Right(FText::FromString(FString(TEXT("\xf0da"))));
FText FEditorFontGlyphs::Caret_Square_O_Down(FText::FromString(FString(TEXT("\xf150"))));
FText FEditorFontGlyphs::Caret_Square_O_Left(FText::FromString(FString(TEXT("\xf191"))));
FText FEditorFontGlyphs::Caret_Square_O_Right(FText::FromString(FString(TEXT("\xf152"))));
FText FEditorFontGlyphs::Caret_Square_O_Up(FText::FromString(FString(TEXT("\xf151"))));
FText FEditorFontGlyphs::Caret_Up(FText::FromString(FString(TEXT("\xf0d8"))));
FText FEditorFontGlyphs::Check(FText::FromString(FString(TEXT("\xf00c"))));
FText FEditorFontGlyphs::Check_Circle(FText::FromString(FString(TEXT("\xf058"))));
FText FEditorFontGlyphs::Circle(FText::FromString(FString(TEXT("\xf111"))));
FText FEditorFontGlyphs::Circle_O(FText::FromString(FString(TEXT("\xf10c"))));
FText FEditorFontGlyphs::Clock_O(FText::FromString(FString(TEXT("\xf017"))));
FText FEditorFontGlyphs::Cog(FText::FromString(FString(TEXT("\xf013"))));
FText FEditorFontGlyphs::Cogs(FText::FromString(FString(TEXT("\xf085"))));
FText FEditorFontGlyphs::Crosshairs(FText::FromString(FString(TEXT("\xf05b"))));
FText FEditorFontGlyphs::Database(FText::FromString(FString(TEXT("\xf1c0"))));
FText FEditorFontGlyphs::Download(FText::FromString(FString(TEXT("\xf019"))));
FText FEditorFontGlyphs::Eraser(FText::FromString(FString(TEXT("\xf12d"))));
FText FEditorFontGlyphs::Exchange(FText::FromString(FString(TEXT("\xf0ec"))));
FText FEditorFontGlyphs::Exclamation(FText::FromString(FString(TEXT("\xf12a"))));
FText FEditorFontGlyphs::Exclamation_Triangle(FText::FromString(FString(TEXT("\xf071"))));
FText FEditorFontGlyphs::Expand(FText::FromString(FString(TEXT("\xf065"))));
FText FEditorFontGlyphs::External_Link(FText::FromString(FString(TEXT("\xf08e"))));
FText FEditorFontGlyphs::Eye(FText::FromString(FString(TEXT("\xf06e"))));
FText FEditorFontGlyphs::Eye_Slash(FText::FromString(FString(TEXT("\xf070"))));
FText FEditorFontGlyphs::Fast_Forward(FText::FromString(FString(TEXT("\xf050"))));
FText FEditorFontGlyphs::File(FText::FromString(FString(TEXT("\xf15b"))));
FText FEditorFontGlyphs::Film(FText::FromString(FString(TEXT("\xf008"))));
FText FEditorFontGlyphs::Filter(FText::FromString(FString(TEXT("\xf0b0"))));
FText FEditorFontGlyphs::Floppy_O(FText::FromString(FString(TEXT("\xf0c7"))));
FText FEditorFontGlyphs::Folder(FText::FromString(FString(TEXT("\xf07b"))));
FText FEditorFontGlyphs::Folder_Open(FText::FromString(FString(TEXT("\xf07c"))));
FText FEditorFontGlyphs::Hourglass(FText::FromString(FString(TEXT("\xf254"))));
FText FEditorFontGlyphs::Hourglass_O(FText::FromString(FString(TEXT("\xf250"))));
FText FEditorFontGlyphs::Info(FText::FromString(FString(TEXT("\xf129"))));
FText FEditorFontGlyphs::Info_Circle(FText::FromString(FString(TEXT("\xf05a"))));
FText FEditorFontGlyphs::Level_Down(FText::FromString(FString(TEXT("\xf149"))));
FText FEditorFontGlyphs::Level_Up(FText::FromString(FString(TEXT("\xf148"))));
FText FEditorFontGlyphs::Lightbulb_O(FText::FromString(FString(TEXT("\xf0eb"))));
FText FEditorFontGlyphs::Line_Chart(FText::FromString(FString(TEXT("\xf201"))));
FText FEditorFontGlyphs::Link(FText::FromString(FString(TEXT("\xf0c1"))));
FText FEditorFontGlyphs::Lock(FText::FromString(FString(TEXT("\xf023"))));
FText FEditorFontGlyphs::Long_Arrow_Down(FText::FromString(FString(TEXT("\xf175"))));
FText FEditorFontGlyphs::Long_Arrow_Right(FText::FromString(FString(TEXT("\xf178"))));
FText FEditorFontGlyphs::Map(FText::FromString(FString(TEXT("\xf279"))));
FText FEditorFontGlyphs::Map_Marker(FText::FromString(FString(TEXT("\xf041"))));
FText FEditorFontGlyphs::Minus(FText::FromString(FString(TEXT("\xf068"))));
FText FEditorFontGlyphs::Minus_Circle(FText::FromString(FString(TEXT("\xf056"))));
FText FEditorFontGlyphs::Paper_Plane(FText::FromString(FString(TEXT("\xf1d8"))));
FText FEditorFontGlyphs::Pause(FText::FromString(FString(TEXT("\xf04c"))));
FText FEditorFontGlyphs::Pause_Circle(FText::FromString(FString(TEXT("\xf28b"))));
FText FEditorFontGlyphs::Pencil(FText::FromString(FString(TEXT("\xf040"))));
FText FEditorFontGlyphs::Pencil_Square(FText::FromString(FString(TEXT("\xf14b"))));
FText FEditorFontGlyphs::Play(FText::FromString(FString(TEXT("\xf04b"))));
FText FEditorFontGlyphs::Play_Circle(FText::FromString(FString(TEXT("\xf144"))));
FText FEditorFontGlyphs::Plug(FText::FromString(FString(TEXT("\xf1e6"))));
FText FEditorFontGlyphs::Plus(FText::FromString(FString(TEXT("\xf067"))));
FText FEditorFontGlyphs::Plus_Circle(FText::FromString(FString(TEXT("\xf055"))));
FText FEditorFontGlyphs::Question(FText::FromString(FString(TEXT("\xf128"))));
FText FEditorFontGlyphs::Random(FText::FromString(FString(TEXT("\xf074"))));
FText FEditorFontGlyphs::Recycle(FText::FromString(FString(TEXT("\xf1b8"))));
FText FEditorFontGlyphs::Refresh(FText::FromString(FString(TEXT("\xf021"))));
FText FEditorFontGlyphs::Share(FText::FromString(FString(TEXT("\xf064"))));
FText FEditorFontGlyphs::Share_Alt(FText::FromString(FString(TEXT("\xf1e0"))));
FText FEditorFontGlyphs::Sign_In(FText::FromString(FString(TEXT("\xf090"))));
FText FEditorFontGlyphs::Sign_Out(FText::FromString(FString(TEXT("\xf08b"))));
FText FEditorFontGlyphs::Square(FText::FromString(FString(TEXT("\xf0c8"))));
FText FEditorFontGlyphs::Star(FText::FromString(FString(TEXT("\xf005"))));
FText FEditorFontGlyphs::Step_Forward(FText::FromString(FString(TEXT("\xf051"))));
FText FEditorFontGlyphs::Stop(FText::FromString(FString(TEXT("\xf04d"))));
FText FEditorFontGlyphs::Terminal(FText::FromString(FString(TEXT("\xf120"))));
FText FEditorFontGlyphs::Th(FText::FromString(FString(TEXT("\xf00a"))));
FText FEditorFontGlyphs::Thumb_Tack(FText::FromString(FString(TEXT("\xf08d"))));
FText FEditorFontGlyphs::Times(FText::FromString(FString(TEXT("\xf00d"))));
FText FEditorFontGlyphs::Times_Circle(FText::FromString(FString(TEXT("\xf057"))));
FText FEditorFontGlyphs::Trash(FText::FromString(FString(TEXT("\xf1f8"))));
FText FEditorFontGlyphs::Undo(FText::FromString(FString(TEXT("\xf0e2"))));
FText FEditorFontGlyphs::Unlock(FText::FromString(FString(TEXT("\xf09c"))));
FText FEditorFontGlyphs::Video_Camera(FText::FromString(FString(TEXT("\xf03d"))));
