// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IEditorStyleModule.h"
#include "StarshipStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/StyleColors.h"


/**
 * Implements the Editor style module, loaded by SlateApplication dynamically at startup.
 */
class FEditorStyleModule
	: public IEditorStyleModule
{
public:

	// IEditorStyleModule interface

	virtual void StartupModule( ) override
	{
#if ALLOW_THEMES
		USlateThemeManager::Get().ValidateActiveTheme();
#endif

		FStarshipEditorStyle::Initialize();

		// set the application style to be the editor style
		FAppStyle::SetAppStyleSetName(FStarshipEditorStyle::GetStyleSetName());
	}

	virtual void ShutdownModule( ) override
	{
		FStarshipEditorStyle::Shutdown();
	}

	// End IModuleInterface interface
};


IMPLEMENT_MODULE(FEditorStyleModule, EditorStyle)
