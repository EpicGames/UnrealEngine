// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorStyleSet.h"

#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"

TSharedPtr< ISlateStyle > FEditorStyle::Instance = NULL;

void FEditorStyle::ResetToDefault()
{
	SetStyle( FCoreStyle::Create("EditorStyle") );
}

void FEditorStyle::SetStyle( const TSharedRef< ISlateStyle >& NewStyle )
{
	if( Instance != NewStyle )
	{
		if ( Instance.IsValid() )
		{
			FSlateStyleRegistry::UnRegisterSlateStyle( *Instance.Get() );
		}

		Instance = NewStyle;

		if ( Instance.IsValid() )
		{
			FSlateStyleRegistry::RegisterSlateStyle( *Instance.Get() );
		}
		else
		{
			ResetToDefault();
		}
	}
}
