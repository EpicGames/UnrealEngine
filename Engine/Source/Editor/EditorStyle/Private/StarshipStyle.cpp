// Copyright Epic Games, Inc. All Rights Reserved.

#include "StarshipStyle.h"
#include "Misc/CommandLine.h"
#include "Styling/StarshipCoreStyle.h"

#include "Settings/EditorStyleSettings.h"

#include "SlateOptMacros.h"
#include "Styling/SlateStyleMacros.h"

#if (WITH_EDITOR || (IS_PROGRAM && PLATFORM_DESKTOP))
	#include "PlatformInfo.h"
#endif
#include "Styling/DepthBarStyle.h"
#include "Styling/ToolBarStyle.h"
#include "Styling/SegmentedControlStyle.h"
#include "Styling/StyleColors.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"

#define LOCTEXT_NAMESPACE "EditorStyle"

FName FStarshipEditorStyle::StyleSetName = TEXT("EditorStyle");

void FStarshipEditorStyle::Initialize()
{
	LLM_SCOPE_BYNAME(TEXT("FStarshipEditorStyle"));

	// The core style must be initialized before the editor style
	FSlateApplication::InitializeCoreStyle();

	const FString ThemesSubDir = TEXT("Slate/Themes");

#if ALLOW_THEMES
	USlateThemeManager::Get().ApplyTheme(USlateThemeManager::Get().GetCurrentTheme().Id);
	//UStyleColorTable::Get().SaveCurrentThemeAs(UStyleColorTable::Get().GetCurrentTheme().Filename);
#endif
	StyleInstance = Create();
	SetStyle(StyleInstance.ToSharedRef());
}

void FStarshipEditorStyle::Shutdown()
{
	StyleInstance.Reset();
}

const FName& FStarshipEditorStyle::GetStyleSetName()
{
	return FStarshipEditorStyle::StyleSetName;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

/* FStarshipEditorStyle static initialization
 *****************************************************************************/

TSharedPtr< FStarshipEditorStyle::FStyle > FStarshipEditorStyle::StyleInstance = NULL;

void FStarshipEditorStyle::FStyle::SetColor(const TSharedRef< FLinearColor >& Source, const FLinearColor& Value)
{
	Source->R = Value.R;
	Source->G = Value.G;
	Source->B = Value.B;
	Source->A = Value.A;
}

bool FStarshipEditorStyle::FStyle::IncludeEditorSpecificStyles()
{
#if IS_PROGRAM
	return true;
#else
	return GIsEditor;
#endif
}

/* FStarshipEditorStyle interface
 *****************************************************************************/

 // Note, these sizes are in Slate Units.
 // Slate Units do NOT have to map to pixels.
const FVector2f FStarshipEditorStyle::FStyle::Icon7x16 = FVector2f(7.0f, 16.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon8x4 = FVector2f(8.0f, 4.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon16x4 = FVector2f(16.0f, 4.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon8x8 = FVector2f(8.0f, 8.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon10x10 = FVector2f(10.0f, 10.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon12x12 = FVector2f(12.0f, 12.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon12x16 = FVector2f(12.0f, 16.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon14x14 = FVector2f(14.0f, 14.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon16x16 = FVector2f(16.0f, 16.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon16x20 = FVector2f(16.0f, 20.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon20x20 = FVector2f(20.0f, 20.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon22x22 = FVector2f(22.0f, 22.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon24x24 = FVector2f(24.0f, 24.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon25x25 = FVector2f(25.0f, 25.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon32x32 = FVector2f(32.0f, 32.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon40x40 = FVector2f(40.0f, 40.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon48x48 = FVector2f(48.0f, 48.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon64x64 = FVector2f(64.0f, 64.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon36x24 = FVector2f(36.0f, 24.0f);
const FVector2f FStarshipEditorStyle::FStyle::Icon128x128 = FVector2f(128.0f, 128.0f);

FStarshipEditorStyle::FStyle::FStyle()
	: FSlateStyleSet(FStarshipEditorStyle::StyleSetName)

	// These are the colors that are updated by the user style customizations
	, SelectionColor_Subdued_LinearRef(MakeShareable(new FLinearColor(0.807f, 0.596f, 0.388f)))
	, HighlightColor_LinearRef( MakeShareable( new FLinearColor(0.068f, 0.068f, 0.068f) ) ) 
	, WindowHighlightColor_LinearRef(MakeShareable(new FLinearColor(0.f,0.f,0.f,0.f)))



	// These are the Slate colors which reference those above; these are the colors to put into the style
	, SelectionColor_Subdued( SelectionColor_Subdued_LinearRef )
	, HighlightColor( HighlightColor_LinearRef )
	, WindowHighlightColor(WindowHighlightColor_LinearRef)
	, InheritedFromBlueprintTextColor(FLinearColor(0.25f, 0.5f, 1.0f))
{
}

FStarshipEditorStyle::FStyle::~FStyle()
{
#if WITH_EDITOR
	// GetMutableDefault is invalid during shutdown as the object system is unloaded before FStarshipEditorStyle
	if(UObjectInitialized() && !IsEngineExitRequested())
	{
		if (UEditorStyleSettings* Settings = GetMutableDefault<UEditorStyleSettings>())
		{
			Settings->OnSettingChanged().Remove(SettingChangedHandler);
		}
	}
	
#endif

}

void FStarshipEditorStyle::FStyle::SettingsChanged(FName PropertyName)
{
		SyncSettings();
}

void FStarshipEditorStyle::FStyle::SyncSettings()
{
	if (UEditorStyleSettings* Settings = GetMutableDefault<UEditorStyleSettings>())
	{
		// The subdued selection color is derived from the selection color
		auto SubduedSelectionColor = Settings->GetSubduedSelectionColor();
		SetColor(SelectionColor_Subdued_LinearRef, SubduedSelectionColor);

		// Sync the window background settings
		FWindowStyle& WindowStyle = const_cast<FWindowStyle&>(FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FWindowStyle>("Window"));
		if (Settings->bEnableEditorWindowBackgroundColor)
		{
			SetColor(WindowHighlightColor_LinearRef, Settings->EditorWindowBackgroundColor);
			WindowTitleOverride->TintColor = WindowHighlightColor_LinearRef;
		}
		else
		{
			SetColor(WindowHighlightColor_LinearRef, FLinearColor(0, 0, 0, 0));
			WindowTitleOverride->TintColor = FStyleColors::Title;
		}
	}
}

void FStarshipEditorStyle::FStyle::SyncParentStyles()
{
	const ISlateStyle* ParentStyle = GetParentStyle();

	// Get the scrollbar style from the core style as it is referenced by the editor style
	ScrollBar = ParentStyle->GetWidgetStyle<FScrollBarStyle>("ScrollBar");
	NoBorder = ParentStyle->GetWidgetStyle<FButtonStyle>("NoBorder");
	NormalFont = ParentStyle->GetFontStyle("NormalFont");
	NormalText = ParentStyle->GetWidgetStyle<FTextBlockStyle>("NormalText");
	Button = ParentStyle->GetWidgetStyle<FButtonStyle>("Button");
	NormalEditableTextBoxStyle = ParentStyle->GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
	NormalTableRowStyle = ParentStyle->GetWidgetStyle<FTableRowStyle>("TableView.Row");

	DefaultForeground = ParentStyle->GetSlateColor("DefaultForeground");
	InvertedForeground = ParentStyle->GetSlateColor("InvertedForeground");

	SelectorColor = ParentStyle->GetSlateColor("SelectorColor");
	SelectionColor = ParentStyle->GetSlateColor("SelectionColor");
	SelectionColor_Inactive = ParentStyle->GetSlateColor("SelectionColor_Inactive");
	SelectionColor_Pressed = ParentStyle->GetSlateColor("SelectionColor_Pressed");
}

static void AuditDuplicatedCoreStyles(const ISlateStyle& EditorStyle)
{
	const ISlateStyle& CoreStyle = FStarshipCoreStyle::GetCoreStyle();
	TSet<FName> CoreStyleKeys = CoreStyle.GetStyleKeys();

	TSet<FName> EditorStyleKeys = EditorStyle.GetStyleKeys();

	TSet<FName> DuplicatedNames = CoreStyleKeys.Intersect(EditorStyleKeys);

	DuplicatedNames.Sort(FNameLexicalLess());
	for (FName& Name : DuplicatedNames)
	{
		UE_LOGF(LogSlate, Log, "%ls", *Name.ToString());
	}
}

void FStarshipEditorStyle::FStyle::Initialize()
{
	SetParentStyleName("CoreStyle");

	// Sync styles from the parent style that will be used as templates for styles defined here
	SyncParentStyles();


	SetContentRoot( FPaths::EngineContentDir() / TEXT("Editor/Slate") );
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	SetupGeneralStyles();
	SetupLevelGeneralStyles();
	SetupWorldBrowserStyles();
	SetupWorldPartitionStyles();
	SetupSequencerStyles();
	SetupViewportStyles();
	SetupDepthBarStyles();
	SetupMenuBarStyles();
	SetupGeneralIcons();
	SetupWindowStyles();
	SetupPropertyEditorStyles();

	// Avoid polluting the game texture atlas with non-core editor style items when not the editor (or a standalone application, like UFE)
	if (!IncludeEditorSpecificStyles())
	{
		return;
	}

	SetupProjectBadgeStyle();
	SetupDockingStyles();
	SetupTutorialStyles();
	SetupProfilerStyle();
	SetupGraphEditorStyles();
	SetupLevelEditorStyle();
	SetupPersonaStyle();
	SetupClassThumbnailOverlays();
	SetupClassIconsAndThumbnails();
	SetupContentBrowserStyle();
	SetupLandscapeEditorStyle();
	SetupToolkitStyles();
	SetupTranslationEditorStyles();
	SetupLocalizationDashboardStyles();
	SetupUnsavedAssetsStyles();
	SetupSourceControlStyles();
	SetupAutomationStyles();
	SetupUMGEditorStyles();
	SetupHomeScreenStyles();
	SetupMyBlueprintStyles();
	SetupStatusBarStyle();
	SetupColorPickerStyle();
	SetupSourceCodeStyles();

//	LogUnusedBrushResources();

	AuditDuplicatedCoreStyles(*this);
	
	SyncSettings();

#if WITH_EDITOR
	if (UEditorStyleSettings* Settings = GetMutableDefault<UEditorStyleSettings>())
	{
		SettingChangedHandler = Settings->OnSettingChanged().AddRaw(this, &FStarshipEditorStyle::FStyle::SettingsChanged);
	}
#endif

}

void FStarshipEditorStyle::FStyle::SetupGeneralStyles()
{
	
	// Normal Text
	{
		Set( "RichTextBlock.TextHighlight", FTextBlockStyle(NormalText)
			.SetColorAndOpacity( FLinearColor( 1.0f, 1.0f, 1.0f ) ) );
		Set("RichTextBlock.DarkText", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f)));
		Set("RichTextBlock.ForegroundText", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FStyleColors::Secondary));
		Set( "RichTextBlock.BoldDarkText", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize))
			.SetColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f)));
		Set( "RichTextBlock.Bold", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize )) );
		Set( "RichTextBlock.BoldHighlight", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize ))
			.SetColorAndOpacity( FLinearColor( 1.0f, 1.0f, 1.0f ) ) );
		Set("RichTextBlock.Italic", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Italic", FStarshipCoreStyle::RegularTextSize)));
		Set("RichTextBlock.ItalicHighlight", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Italic", FStarshipCoreStyle::RegularTextSize))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f)));

		Set( "TextBlock.HighlightShape",  new BOX_BRUSH( "Common/TextBlockHighlightShape", FMargin(3.f/8.f) ));
		Set( "TextBlock.HighlighColor", FLinearColor( 0.02f, 0.3f, 0.0f ) );

		Set("TextBlock.ShadowedText", FTextBlockStyle(NormalText)
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f)));

		Set("TextBlock.ShadowedTextWarning", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FStyleColors::Warning)
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f)));

		Set("NormalText.Subdued", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FSlateColor::UseSubduedForeground()));

		Set("NormalText.Important", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

		Set("SmallText.Subdued", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", FStarshipCoreStyle::SmallTextSize))
			.SetColorAndOpacity(FSlateColor::UseSubduedForeground()));

		Set("TinyText", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", FStarshipCoreStyle::SmallTextSize)));

		Set("TinyText.Subdued", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", FStarshipCoreStyle::SmallTextSize))
			.SetColorAndOpacity(FSlateColor::UseSubduedForeground()));

		Set("LargeText", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 11))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));
	}

	// EULA RichText
	{
		Set("EULA.Header", FTextBlockStyle(FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("Log.Normal"))
			.SetFont(FCoreStyle::GetDefaultFontStyle("Bold", 10)));

		Set("EULA.Highlight", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f)));

		Set("EULA.HighlightItalic", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Italic", FStarshipCoreStyle::RegularTextSize))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f)));

		const FButtonStyle EulaHyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor(0.25f, 0.5f, 1.0f)))
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor(0.25f, 0.5f, 1.0f)));

		const FTextBlockStyle EulaHyperlinkText = FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FLinearColor(0.25f, 0.5f, 1.0f));

		Set("EULA.Hyperlink", FHyperlinkStyle()
			.SetUnderlineStyle(EulaHyperlinkButton)
			.SetTextStyle(EulaHyperlinkText)
			.SetPadding(FMargin(0.0f)));

	}

	// Rendering resources that never change
	{
		Set( "None", new FSlateNoResource() );
	}



	Set( "WideDash.Horizontal", new CORE_IMAGE_BRUSH("Starship/Common/Dash_Horizontal", FVector2f(10.f, 1.f), FLinearColor::White, ESlateBrushTileType::Horizontal));
	Set( "WideDash.Vertical", new CORE_IMAGE_BRUSH("Starship/Common/Dash_Vertical", FVector2f(1.f, 10.f), FLinearColor::White, ESlateBrushTileType::Vertical));

	Set("DropTarget.Background", new CORE_BOX_BRUSH("Starship/Common/DropTargetBackground", FMargin(6.0f / 64.0f)));

	Set("ThinLine.Horizontal", new IMAGE_BRUSH("Common/ThinLine_Horizontal", FVector2f(11.f, 2.f), FLinearColor::White, ESlateBrushTileType::Horizontal));


	// Buttons that only provide a hover hint.
	HoverHintOnly = FButtonStyle()
			.SetNormal( FSlateNoResource() )
			.SetHovered( BOX_BRUSH( "Common/ButtonHoverHint", FMargin(4.f/16.0f), FLinearColor(1.f,1.f,1.f,0.15f) ) )
			.SetPressed( BOX_BRUSH( "Common/ButtonHoverHint", FMargin(4.f/16.0f), FLinearColor(1.f,1.f,1.f,0.25f) ) )
			.SetNormalPadding( FMargin(0.f,0.f,0.f,1.f) )
			.SetPressedPadding( FMargin(0.f,1.f,0.f,0.f) );
	Set( "HoverHintOnly", HoverHintOnly );


	FButtonStyle SimpleSharpButton = FButtonStyle()
		.SetNormal(BOX_BRUSH("Common/Button/simple_sharp_normal", FMargin(4.f / 16.0f), FLinearColor::White))
		.SetHovered(BOX_BRUSH("Common/Button/simple_sharp_hovered", FMargin(4.f / 16.0f), FLinearColor::White))
		.SetPressed(BOX_BRUSH("Common/Button/simple_sharp_hovered", FMargin(4.f / 16.0f), FLinearColor::White))
		.SetNormalPadding(FMargin(0.f, 0.f, 0.f, 1.f))
		.SetPressedPadding(FMargin(0.f, 1.f, 0.f, 0.f));
	Set("SimpleSharpButton", SimpleSharpButton);

	FButtonStyle SimpleRoundButton = FButtonStyle()
		.SetNormal(BOX_BRUSH("Common/Button/simple_round_normal", FMargin(4.f / 16.0f), FLinearColor::White))
		.SetHovered(BOX_BRUSH("Common/Button/simple_round_hovered", FMargin(4.f / 16.0f), FLinearColor::White))
		.SetPressed(BOX_BRUSH("Common/Button/simple_round_hovered", FMargin(4.f / 16.0f), FLinearColor::White))
		.SetNormalPadding(FMargin(0, 0, 0, 1))
		.SetPressedPadding(FMargin(0, 1, 0, 0));
	Set("SimpleRoundButton", SimpleRoundButton);

	// Common glyphs
	{
		Set( "Symbols.SearchGlass", new IMAGE_BRUSH( "Common/SearchGlass", Icon16x16 ) );
		Set( "Symbols.X", new IMAGE_BRUSH( "Common/X", Icon16x16 ) );
		Set( "Symbols.VerticalPipe", new BOX_BRUSH( "Common/VerticalPipe", FMargin(0.f) ) );
		Set( "Symbols.UpArrow", new IMAGE_BRUSH( "Common/UpArrow", Icon8x8 ) );
		Set( "Symbols.DoubleUpArrow", new IMAGE_BRUSH( "Common/UpArrow2", Icon8x8 ) );
		Set( "Symbols.DownArrow", new IMAGE_BRUSH( "Common/DownArrow", Icon8x8 ) );
		Set( "Symbols.DoubleDownArrow", new IMAGE_BRUSH( "Common/DownArrow2", Icon8x8 ) );
		Set( "Symbols.RightArrow", new IMAGE_BRUSH("Common/SubmenuArrow", Icon8x8));
		Set( "Symbols.LeftArrow", new IMAGE_BRUSH("Common/LeftArrow", Icon8x8));
		Set( "Symbols.Check", new IMAGE_BRUSH( "Common/Check", Icon16x16 ) );
	}

	// Common icons
	{
		Set("Icons.Contact", new IMAGE_BRUSH( "Icons/icon_mail_16x", Icon16x16 ) );
		Set("Icons.Crop", new IMAGE_BRUSH_SVG("Starship/Common/Crop", Icon16x16));
		Set("Icons.Fullscreen", new IMAGE_BRUSH_SVG("Starship/Common/EnableFullscreen", Icon16x16));
		Set("Icons.Save", new IMAGE_BRUSH_SVG( "Starship/Common/SaveCurrent", Icon16x16 ) );
		Set("Icons.SaveChanged", new IMAGE_BRUSH_SVG( "Starship/Common/SaveChanged", Icon16x16 ) );

		Set("Icons.DirtyBadge", new IMAGE_BRUSH_SVG("Starship/Common/DirtyBadge", Icon12x12));
		Set("Icons.SaveableBadge", new IMAGE_BRUSH_SVG("Starship/Common/FileBadge", Icon16x16, FLinearColor(0.3f, 0.3f, 0.3f, 0.3f)));
		Set("Icons.MakeStaticMesh", new IMAGE_BRUSH_SVG("Starship/Common/MakeStaticMesh", Icon16x16));
		Set("Icons.Documentation", new IMAGE_BRUSH_SVG("Starship/Common/Documentation", Icon16x16));
		Set("Icons.Support", new IMAGE_BRUSH_SVG("Starship/Common/Support", Icon16x16));
		Set("Icons.Package", new IMAGE_BRUSH_SVG("Starship/Common/ProjectPackage", Icon16x16));
		Set("Icons.Comment", new IMAGE_BRUSH_SVG("Starship/Common/Comment", Icon16x16));
		Set("Icons.SelectInViewport", new IMAGE_BRUSH_SVG("Starship/Common/SelectInViewport", Icon16x16));
		Set("Icons.BrowseContent", new IMAGE_BRUSH_SVG("Starship/Common/BrowseContent", Icon16x16));
		Set("Icons.Use", new IMAGE_BRUSH_SVG("Starship/Common/use-circle", Icon16x16));
		Set("Icons.Next", new IMAGE_BRUSH_SVG("Starship/Common/NextArrow", Icon16x16));
		Set("Icons.Previous", new IMAGE_BRUSH_SVG("Starship/Common/PreviousArrow", Icon16x16));
		Set("Icons.Visibility", new IMAGE_BRUSH_SVG("Starship/Common/Visibility", Icon20x20));
		Set("Icons.World", new IMAGE_BRUSH_SVG("Starship/Common/World", Icon20x20));
		Set("Icons.Details", new IMAGE_BRUSH_SVG("Starship/Common/Details", Icon16x16));
		Set("Icons.Convert", new IMAGE_BRUSH_SVG("Starship/Common/convert", Icon20x20));
		Set("Icons.Adjust", new IMAGE_BRUSH_SVG("Starship/Common/Adjust", Icon16x16));
		Set("Icons.PlaceActors", new IMAGE_BRUSH_SVG("Starship/Common/PlaceActors", Icon16x16));
		Set("Icons.ReplaceActor", new IMAGE_BRUSH_SVG("Starship/Common/ReplaceActors", Icon16x16));
		Set("Icons.GroupActors", new IMAGE_BRUSH_SVG("Starship/Common/GroupActors", Icon16x16));
		Set("Icons.FrameActor", new IMAGE_BRUSH_SVG("Starship/Common/FrameActor", Icon16x16));
		Set("Icons.Transform", new IMAGE_BRUSH_SVG("Starship/Common/transform-local", Icon16x16));
		Set("Icons.SetShowPivot", new IMAGE_BRUSH_SVG("Starship/Common/SetShowPivot", Icon16x16));
		Set("Icons.Snap", new IMAGE_BRUSH_SVG("Starship/Common/Snap", Icon16x16));
		Set("Icons.Event", new IMAGE_BRUSH_SVG("Starship/Common/Event", Icon16x16));
		Set("Icons.JumpToEvent", new IMAGE_BRUSH_SVG("Starship/Common/JumpToEvent", Icon16x16));
		Set("Icons.Level", new IMAGE_BRUSH_SVG("Starship/Common/Levels", Icon16x16));
		Set("Icons.Play", new IMAGE_BRUSH_SVG("Starship/Common/play", Icon16x16));
		Set("Icons.Localization", new IMAGE_BRUSH_SVG("Starship/Common/LocalizationDashboard", Icon16x16));
		Set("Icons.Audit", new IMAGE_BRUSH_SVG("Starship/Common/AssetAudit", Icon16x16));
		Set("Icons.Blueprint", new IMAGE_BRUSH_SVG("Starship/Common/blueprint", Icon16x16));
		Set("Icons.Color", new IMAGE_BRUSH_SVG("Starship/Common/color", Icon16x16));
		Set("Icons.LOD", new IMAGE_BRUSH_SVG("Starship/Common/LOD", Icon16x16));
		Set("Icons.SkeletalMesh", new IMAGE_BRUSH_SVG("Starship/Common/SkeletalMesh", Icon16x16));
		Set("Icons.OpenInExternalEditor", new IMAGE_BRUSH_SVG("Starship/Common/OpenInExternalEditor", Icon16x16));
		Set("Icons.OpenSourceLocation", new IMAGE_BRUSH_SVG("Starship/Common/OpenSourceLocation", Icon16x16));
		Set("Icons.OpenInBrowser", new IMAGE_BRUSH_SVG("Starship/Common/WebBrowser", Icon16x16));
		Set("Icons.OpenFile", new IMAGE_BRUSH_SVG("Starship/Common/OpenFile", Icon16x16));
		Set("Icons.Find", new IMAGE_BRUSH_SVG("Starship/Common/Find", Icon16x16));
		Set("Icons.Validate", new IMAGE_BRUSH_SVG("Starship/Common/validate", Icon16x16));
		Set("Icons.Pinned", new IMAGE_BRUSH_SVG("Starship/Common/Pinned", Icon16x16));
		Set("Icons.Unpinned", new IMAGE_BRUSH_SVG("Starship/Common/Unpinned", Icon16x16));
		Set("Icons.Tools", new IMAGE_BRUSH_SVG("Starship/Common/EditorModes", Icon16x16));
		Set("Icons.Clipboard", new IMAGE_BRUSH_SVG("Starship/Common/Clipboard", Icon16x16));
		Set("Icons.HollowHeart", new IMAGE_BRUSH_SVG("Starship/Common/HollowHeart", Icon16x16));
		Set("Icons.Heart", new IMAGE_BRUSH_SVG("Starship/Common/Heart", Icon16x16));
		Set("Icons.RecentAssets", new IMAGE_BRUSH_SVG("Starship/Common/RecentAssets", Icon16x16));
		Set("Icons.Scalability", new IMAGE_BRUSH_SVG("Starship/Common/Scalability_16", Icon20x20));
		Set("Icons.EditHierarchy", new IMAGE_BRUSH_SVG("Starship/Common/EditHierarchy_16", Icon16x16));
		Set("Icons.ViewportScalability", new IMAGE_BRUSH_SVG("Starship/Common/ViewportScalability_16", Icon16x16));
		Set("Icons.ViewportScalabilityReset", new IMAGE_BRUSH_SVG("Starship/Common/ViewportScalabilityReset_16", Icon16x16));
		Set("Icons.EllipsisVerticalNarrow",
			new CORE_IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 24.f)));

		Set("Icons.Toolbar.Play", new IMAGE_BRUSH_SVG("Starship/Common/play", Icon20x20));
		Set("Icons.Toolbar.Pause", new IMAGE_BRUSH_SVG("Starship/MainToolbar/pause", Icon20x20));
		Set("Icons.Toolbar.Stop", new CORE_IMAGE_BRUSH_SVG("Starship/Common/stop", Icon20x20));
		Set("Icons.Toolbar.Settings", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/Settings", Icon20x20));
		Set("Icons.Toolbar.Details", new IMAGE_BRUSH_SVG("Starship/Common/Details", Icon16x16));
		Set("Icons.Toolbar.Import", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import_20", Icon20x20));
		Set("Icons.Toolbar.Reimport", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));
		Set("Icons.Toolbar.Export", new CORE_IMAGE_BRUSH_SVG("Starship/Common/export_20", Icon20x20));

		Set("Icons.Alert.Solid", new IMAGE_BRUSH_SVG("Starship/Common/AlertTriangleSolid", Icon16x16, FStyleColors::Foreground));
		Set("Icons.Alert.Badge", new IMAGE_BRUSH_SVG("Starship/Common/AlertTriangleBadge", Icon16x16, FStyleColors::Foreground));
		Set("Icons.Alert.Background", new IMAGE_BRUSH_SVG("Starship/Common/AlertTriangleBackground", Icon16x16, FStyleColors::Hover2));
		Set("Icons.Error.Solid", new IMAGE_BRUSH_SVG("Starship/Common/AlertTriangleSolid", Icon16x16, FStyleColors::Error));
		Set("Icons.Error.Background", new IMAGE_BRUSH_SVG("Starship/Common/AlertTriangleBackground", Icon16x16, FStyleColors::Error));
		Set("Icons.Warning.Solid", new IMAGE_BRUSH_SVG("Starship/Common/AlertTriangleSolid", Icon16x16, FStyleColors::Warning));
		Set("Icons.Warning.Background", new IMAGE_BRUSH_SVG("Starship/Common/AlertTriangleBackground", Icon16x16, FStyleColors::Warning));
	}

	// Theme Icons
	{
		Set("Themes.Import", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import", Icon16x16));
		Set("Themes.Export", new CORE_IMAGE_BRUSH_SVG("Starship/Common/export", Icon16x16));
	}

	// Typed Elements Icons
	{
		Set("Icons.PromoteElements", new IMAGE_BRUSH_SVG("Starship/Common/convert", Icon20x20));
		Set("Icons.DemoteElements", new IMAGE_BRUSH_SVG("Starship/Common/convert", Icon20x20));
	}

	// Reference Viewer Icons
	{
		Set("ReferenceViewer.IndirectReference", new IMAGE_BRUSH_SVG("Starship/ReferenceViewer/IndirectRef", Icon16x16));
	}

	Set("UnrealDefaultThumbnail", new IMAGE_BRUSH("Starship/Common/Unreal_DefaultThumbnail", FVector2f(256.f, 256.f)));

	FSlateImageBrush* RoundedDefaultThumbnail = new IMAGE_BRUSH("Starship/Common/Unreal_DefaultThumbnail", FVector2f(256.f, 256.f));
	RoundedDefaultThumbnail->OutlineSettings.CornerRadii = FVector4(4, 4, 0, 0);
	RoundedDefaultThumbnail->OutlineSettings.RoundingType = ESlateBrushRoundingType::FixedRadius;
	RoundedDefaultThumbnail->DrawAs = ESlateBrushDrawType::RoundedBox;
	Set("UnrealDefaultThumbnailRounded", RoundedDefaultThumbnail);

	Set( "WarningStripe", new IMAGE_BRUSH( "Common/WarningStripe", FVector2f(20.f,6.f), FLinearColor::White, ESlateBrushTileType::Horizontal ) );
	
	Set("RoundedWarning", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Warning, 1.0f));
	Set("RoundedError", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Error, 1.0f));

	Set( "Button.Disabled", new BOX_BRUSH( "Common/Button_Disabled", 8.0f/32.0f ) );
	

	// Toggle button
	{
		Set( "ToggleButton", FButtonStyle(Button)
			.SetNormal(FSlateNoResource())
			.SetHovered(BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ))
			.SetPressed(BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ))
		);

		//FSlateColorBrush(FLinearColor::White)

		Set("RoundButton", FButtonStyle(Button)
			.SetNormal(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(1.f, 1.f, 1.f, 0.1f)))
			.SetHovered(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
			.SetPressed(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			);

		Set("FlatButton", FButtonStyle(Button)
			.SetNormal(FSlateNoResource())
			.SetHovered(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor))
			.SetPressed(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor_Pressed))
			);

		Set("FlatButton.Dark", FButtonStyle(Button)
			.SetNormal(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, FLinearColor(0.125f, 0.125f, 0.125f, 0.8f)))
			.SetHovered(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor))
			.SetPressed(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor_Pressed))
			);

		Set("FlatButton.DarkGrey", FButtonStyle(Button)
			.SetNormal(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, FLinearColor(0.05f, 0.05f, 0.05f, 0.8f)))
			.SetHovered(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor))
			.SetPressed(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor_Pressed))
		);


		Set("FlatButton.Default", GetWidgetStyle<FButtonStyle>("FlatButton.Dark"));

		Set("FlatButton.DefaultTextStyle", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 10))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.9f)));

		struct ButtonColor
		{
		public:
			FName Name;
			FLinearColor Normal;
			FLinearColor Hovered;
			FLinearColor Pressed;

			ButtonColor(const FName& InName, const FLinearColor& Color) : Name(InName)
			{
				Normal = Color * 0.8f;
				Normal.A = Color.A;
				Hovered = Color * 1.0f;
				Hovered.A = Color.A;
				Pressed = Color * 0.6f;
				Pressed.A = Color.A;
			}
		};

		TArray< ButtonColor > FlatButtons;
		FlatButtons.Add(ButtonColor("FlatButton.Primary", FLinearColor(0.02899f, 0.19752f, 0.48195f)));
		FlatButtons.Add(ButtonColor("FlatButton.Success", FLinearColor(0.10616f, 0.48777f, 0.10616f)));
		FlatButtons.Add(ButtonColor("FlatButton.Info", FLinearColor(0.10363f, 0.53564f, 0.7372f)));
		FlatButtons.Add(ButtonColor("FlatButton.Warning", FLinearColor(0.87514f, 0.42591f, 0.07383f)));
		FlatButtons.Add(ButtonColor("FlatButton.Danger", FLinearColor(0.70117f, 0.08464f, 0.07593f)));

		for ( const ButtonColor& Entry : FlatButtons )
		{
			Set(Entry.Name, FButtonStyle(Button)
				.SetNormal(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, Entry.Normal))
				.SetHovered(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, Entry.Hovered))
				.SetPressed(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, Entry.Pressed))
				);
		}

		Set("FontAwesome.7", REGULAR_ICON_FONT(7));
		Set("FontAwesome.8", REGULAR_ICON_FONT(8));
		Set("FontAwesome.9", REGULAR_ICON_FONT(9));
		Set("FontAwesome.10", REGULAR_ICON_FONT(10));
		Set("FontAwesome.11", REGULAR_ICON_FONT(11));
		Set("FontAwesome.12", REGULAR_ICON_FONT(12));
		Set("FontAwesome.14", REGULAR_ICON_FONT(14));
		Set("FontAwesome.16", REGULAR_ICON_FONT(16));
		Set("FontAwesome.18", REGULAR_ICON_FONT(18));

		/* Create a checkbox style for "ToggleButton" but with the images used by a normal checkbox (see "Checkbox" below) ... */
		const FCheckBoxStyle CheckboxLookingToggleButtonStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage( IMAGE_BRUSH( "Common/CheckBox", Icon16x16 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/CheckBox", Icon16x16 ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Hovered", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/CheckBox_Checked_Hovered", Icon16x16 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Checked_Hovered", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Checked", Icon16x16 ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined", Icon16x16 ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon16x16 ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetPadding(1.0f);
		/* ... and set new style */
		Set( "CheckboxLookToggleButtonCheckbox", CheckboxLookingToggleButtonStyle );


		Set( "ToggleButton.LabelFont", DEFAULT_FONT( "Regular", 9 ) );
		Set( "ToggleButtonCheckbox.LabelFont", DEFAULT_FONT( "Regular", 9 ) );
	}

	// Combo Button, Combo Box
	{
		// Legacy style; still being used by some editor widgets
		Set( "ComboButton.Arrow", new IMAGE_BRUSH("Common/ComboArrow", Icon8x8 ) );


		FComboButtonStyle ToolbarComboButton = FComboButtonStyle()
			.SetButtonStyle( GetWidgetStyle<FButtonStyle>( "ToggleButton" ) )
			.SetDownArrowImage( IMAGE_BRUSH( "Common/ShadowComboArrow", Icon8x8 ) )
			.SetMenuBorderBrush(FSlateNoResource())
			.SetMenuBorderPadding( FMargin( 0.0f ) );
		Set( "ToolbarComboButton", ToolbarComboButton );

		Set("GenericFilters.ComboButtonStyle", ToolbarComboButton);

		Set("GenericFilters.TextStyle", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 9))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 0.9f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.9f)));

	}

	// Error Reporting
	{
		Set( "InfoReporting.BackgroundColor", FLinearColor(0.1f, 0.33f, 1.0f));

	}

	// EditableTextBox
	{
		Set( "EditableTextBox.Background.Normal", new BOX_BRUSH( "Common/TextBox", FMargin(4.0f/16.0f) ) );
		Set( "EditableTextBox.Background.Hovered", new BOX_BRUSH( "Common/TextBox_Hovered", FMargin(4.0f/16.0f) ) );
		Set( "EditableTextBox.Background.Focused", new BOX_BRUSH( "Common/TextBox_Hovered", FMargin(4.0f/16.0f) ) );
		Set( "EditableTextBox.Background.ReadOnly", new BOX_BRUSH( "Common/TextBox_ReadOnly", FMargin(4.0f/16.0f) ) );
		Set( "EditableTextBox.BorderPadding", FMargin(4.0f, 2.0f) );
	}

	// EditableTextBox Special
	{
		FSlateBrush* SpecialEditableTextImageNormal = new BOX_BRUSH( "Common/TextBox_Special", FMargin(8.0f/32.0f) );
		Set( "SpecialEditableTextImageNormal", SpecialEditableTextImageNormal );

		const FEditableTextBoxStyle SpecialEditableTextBoxStyle = FEditableTextBoxStyle()
			.SetTextStyle(NormalText)
			.SetBackgroundImageNormal( *SpecialEditableTextImageNormal )
			.SetBackgroundImageHovered( BOX_BRUSH( "Common/TextBox_Special_Hovered", FMargin(8.0f/32.0f) ) )
			.SetBackgroundImageFocused( BOX_BRUSH( "Common/TextBox_Special_Hovered", FMargin(8.0f/32.0f) ) )
			.SetBackgroundImageReadOnly( BOX_BRUSH( "Common/TextBox_ReadOnly", FMargin(4.0f/16.0f) ) )
			.SetScrollBarStyle( ScrollBar );
		Set( "SpecialEditableTextBox", SpecialEditableTextBoxStyle );

		Set( "SearchBox.ActiveBorder", new BOX_BRUSH( "Common/TextBox_Special_Active", FMargin(8.0f/32.0f) ) );
	}

	// Filtering/Searching feedback
	{
		const FLinearColor ActiveFilterColor = FLinearColor(1.0f,0.55f,0.0f,1.0f);
		Set("Searching.SearchActiveTab",    new FSlateNoResource());
		Set("Searching.SearchActiveBorder", new FSlateRoundedBoxBrush(FLinearColor::Transparent, 0.0f, FStyleColors::Primary, 1.f));
	}



	// Images sizes are specified in Slate Screen Units. These do not necessarily correspond to pixels!
	// An IMAGE_BRUSH( "SomeImage", FVector2f(32,32)) will have a desired size of 16x16 Slate Screen Units.
	// This allows the original resource to be scaled up or down as needed.

	Set( "WhiteTexture", new IMAGE_BRUSH( "Old/White", Icon16x16 ) );

	Set( "BoldFont", DEFAULT_FONT( "Bold", FStarshipCoreStyle::RegularTextSize ) );

	Set( "Editor.AppIcon", new IMAGE_BRUSH( "Icons/EditorAppIcon", Icon24x24) );

	Set( "MarqueeSelection", new BORDER_BRUSH( "Old/DashedBorder", FMargin(6.0f/32.0f) ) );

	Set( "GenericPlay", new IMAGE_BRUSH( "Icons/generic_play_16x", Icon16x16 ) );
	Set( "GenericPause", new IMAGE_BRUSH( "Icons/generic_pause_16x", Icon16x16 ) );
	Set( "GenericStop", new IMAGE_BRUSH( "Icons/generic_stop_16x", Icon16x16 ) );

	Set( "SoftwareCursor_Hand", new IMAGE_BRUSH_SVG( "Icons/HandOpen_24", Icon24x24 ) );
	Set( "SoftwareCursor_Grab", new IMAGE_BRUSH_SVG( "Icons/HandGrab_24", Icon24x24 ) );
	Set( "SoftwareCursor_CardinalCross", new IMAGE_BRUSH( "Icons/cursor_cardinal_cross", Icon24x24 ) );
	Set( "SoftwareCursor_UpDown", new IMAGE_BRUSH( "Icons/cursor_updown", Icon16x20 ) );

	Set( "Border", new BOX_BRUSH( "Old/Border", 4.0f/16.0f ) );
	
	Set( "NoteBorder", new BOX_BRUSH( "Old/NoteBorder", FMargin(15.0f/40.0f, 15.0f/40.0f) ) );
	
	Set( "FilledBorder", new BOX_BRUSH( "Old/FilledBorder", 4.0f/16.0f ) );

	Set( "GenericViewButton", new IMAGE_BRUSH( "Icons/view_button", Icon20x20 ) );

	Set("GenericLink", new IMAGE_BRUSH("Common/link", Icon16x16));

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	{
		// Dark Hyperlink - for use on light backgrounds
		FButtonStyle DarkHyperlinkButton = FButtonStyle()
			.SetNormal ( BORDER_BRUSH( "Old/HyperlinkDotted", FMargin(0.f,0.f,0.f,3.f/16.0f), FLinearColor::Black ) )
			.SetPressed( FSlateNoResource() )
			.SetHovered( BORDER_BRUSH( "Old/HyperlinkUnderline", FMargin(0.f,0.f,0.f,3.f/16.0f), FLinearColor::Black ) );
		FHyperlinkStyle DarkHyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(DarkHyperlinkButton)
			.SetTextStyle(NormalText)
			.SetPadding(FMargin(0.0f));
		Set("DarkHyperlink", DarkHyperlink);

		// Visible on hover hyper link
		FButtonStyle HoverOnlyHyperlinkButton = FButtonStyle()
			.SetNormal(FSlateNoResource() )
			.SetPressed(FSlateNoResource() )
			.SetHovered(BORDER_BRUSH( "Old/HyperlinkUnderline", FMargin(0.f,0.f,0.f,3.f/16.0f) ) );
		Set("HoverOnlyHyperlinkButton", HoverOnlyHyperlinkButton);

		FHyperlinkStyle HoverOnlyHyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(HoverOnlyHyperlinkButton)
			.SetTextStyle(NormalText)
			.SetPadding(FMargin(0.0f));
		Set("HoverOnlyHyperlink", HoverOnlyHyperlink);
	}


#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

	// Expandable button
	{
		Set( "ExpandableButton.Collapsed", new IMAGE_BRUSH( "Old/ExpansionButton_Collapsed", Icon32x32) );
		Set( "ExpandableButton.Expanded_Left", new IMAGE_BRUSH( "Old/ExpansionButton_ExpandedLeft", Icon32x32) );
		Set( "ExpandableButton.Expanded_Center", new IMAGE_BRUSH( "Old/ExpansionButton_ExpandedMiddle", Icon32x32) );
		Set( "ExpandableButton.Expanded_Right", new IMAGE_BRUSH( "Old/ExpansionButton_ExpandedRight", Icon32x32) );
	}

	// Content reference
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	{
		Set( "ContentReference.Background.Normal", new BOX_BRUSH( "Common/TextBox", FMargin(4.0f/16.0f) ) );
		Set( "ContentReference.Background.Hovered", new BOX_BRUSH( "Common/TextBox_Hovered", FMargin(4.0f/16.0f) ) );
		Set( "ContentReference.BorderPadding", FMargin(4.0f, 2.0f) );
		Set( "ContentReference.FindInContentBrowser", new IMAGE_BRUSH( "Icons/lens_12x", Icon12x12 ) );
		Set( "ContentReference.UseSelectionFromContentBrowser", new IMAGE_BRUSH( "Icons/assign_12x", Icon12x12 ) );
		Set( "ContentReference.PickAsset", new IMAGE_BRUSH( "Icons/pillarray_16x", Icon12x12 ) );
		Set( "ContentReference.Clear", new IMAGE_BRUSH( "Icons/Cross_12x", Icon12x12 ) );
		Set( "ContentReference.Tools", new IMAGE_BRUSH( "Icons/wrench_16x", Icon12x12 ) );
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

	{
		Set("SystemWideCommands.FindInContentBrowser", new IMAGE_BRUSH_SVG("Starship/Common/ContentBrowser", Icon20x20));
		Set("SystemWideCommands.FindInContentBrowser.Small", new IMAGE_BRUSH_SVG("Starship/Common/ContentBrowser", Icon16x16));
	}
	

	// PList Editor
	{
		Set( "PListEditor.HeaderRow.Background",				new BOX_BRUSH( "Common/TableViewHeader", 4.f/32.f ) );

		Set( "PListEditor.FilteredColor",						new FSlateColorBrush( FColor( 0, 255, 0, 80 ) ) );
		Set( "PListEditor.NoOverlayColor",						new FSlateNoResource() );

		Set( "PListEditor.Button_AddToArray",					new IMAGE_BRUSH( "Icons/PlusSymbol_12x", Icon12x12 ) );
	}
	
	// Material List
	{
		Set( "MaterialList.DragDropBorder", new BOX_BRUSH( "Old/Window/ViewportDebugBorder", 0.8f ) );
		Set( "MaterialList.HyperlinkStyle", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 8 ) ) );
		Set( "MaterialList.HyperlinkStyle.ShadowOffset", FVector2f::ZeroVector );
		Set( "Icons.NaniteBrowseContent", new IMAGE_BRUSH_SVG("Starship/Common/NaniteBrowseContent", Icon16x16));
	}

	// Dialogue Wave Details
	{
		Set( "DialogueWaveDetails.SpeakerToTarget", new IMAGE_BRUSH( "PropertyView/SpeakerToTarget", FVector2f(30.0f, 30.0f) ) );
		Set( "DialogueWaveDetails.HeaderBorder", new BOX_BRUSH( "Common/MenuBarBorder", FMargin(4.0f/16.0f) ) );
		Set( "DialogueWaveDetails.PropertyEditorMenu", new BOX_BRUSH( "Old/Menu_Background", FMargin(8.0f/64.0f) ) );
	}

	// Dialogue Wave Parameter Border
	{
		Set( "DialogueWaveParameter.DropDownBorder", new BOX_BRUSH( "Old/Border", 4.0f/16.0f, FLinearColor::Black) );
	}

#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

	Set( "DashedBorder", new BORDER_BRUSH( "Old/DashedBorder", FMargin(6.0f/32.0f) ) );

	Set( "UniformShadow", new BORDER_BRUSH( "Common/UniformShadow", FMargin( 16.0f / 64.0f ) ) );
	Set( "UniformShadow_Tint", new BORDER_BRUSH( "Common/UniformShadow_Tint", FMargin( 16.0f / 64.0f ) ) );

	// Splitter
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	{
		Set ("SplitterDark", FSplitterStyle()
			.SetHandleNormalBrush( FSlateColorBrush( FLinearColor(FColor( 32, 32, 32) ) ) )
			.SetHandleHighlightBrush( FSlateColorBrush( FLinearColor(FColor( 96, 96, 96) ) ) ) 
			);
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)


	// Lists, Trees
	{

		const FTableViewStyle DefaultTreeViewStyle = FTableViewStyle()
			.SetBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed));
		Set("ListView", DefaultTreeViewStyle);

		const FTableViewStyle DefaultTableViewStyle = FTableViewStyle()
			.SetBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed));
		Set("TreeView", DefaultTableViewStyle);

		Set( "TableView.Row", FTableRowStyle( NormalTableRowStyle) );
		Set( "TableView.DarkRow",FTableRowStyle( NormalTableRowStyle)
			.SetEvenRowBackgroundBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle", Icon16x16))
			.SetEvenRowBackgroundHoveredBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle_Hovered", Icon16x16))
			.SetOddRowBackgroundBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle", Icon16x16))
			.SetOddRowBackgroundHoveredBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle_Hovered", Icon16x16))
			.SetSelectorFocusedBrush(BORDER_BRUSH("Common/Selector", FMargin(4.f / 16.f), SelectorColor))
			.SetActiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
		);
		Set("TableView.NoHoverTableRow", FTableRowStyle(NormalTableRowStyle)
			.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(FSlateNoResource())
			.SetActiveHoveredBrush(FSlateNoResource())
			.SetInactiveHoveredBrush(FSlateNoResource())
			);
	}
	
	// Spinboxes
	{

		// Legacy styles; used by other editor widgets
		Set( "SpinBox.Background", new BOX_BRUSH( "Common/Spinbox", FMargin(4.0f/16.0f) ) );
		Set( "SpinBox.Background.Hovered", new BOX_BRUSH( "Common/Spinbox_Hovered", FMargin(4.0f/16.0f) ) );
		Set( "SpinBox.Fill", new BOX_BRUSH( "Common/Spinbox_Fill", FMargin(4.0f/16.0f, 4.0f/16.0f, 8.0f/16.0f, 4.0f/16.0f) ) );
		Set( "SpinBox.Fill.Hovered", new BOX_BRUSH( "Common/Spinbox_Fill_Hovered", FMargin(4.0f/16.0f) ) );
		Set( "SpinBox.Arrows", new IMAGE_BRUSH( "Common/SpinArrows", Icon12x12 ) );
		Set( "SpinBox.TextMargin", FMargin(1.0f,2.0f) );
	}

	// Throbber
	{
		Set( "SmallThrobber.Chunk", new IMAGE_BRUSH( "Common/ThrobberPiece_Small", FVector2f(8.f,16.f) ) );
	}

	{
		Set("CurveEd.TimelineArea", new IMAGE_BRUSH("Old/White", Icon16x16, FLinearColor(1.f, 1.f, 1.f, 0.25f)));
		Set("CurveEd.FitHorizontal", new IMAGE_BRUSH("Icons/FitHorz_16x", Icon16x16));
		Set("CurveEd.FitVertical", new IMAGE_BRUSH("Icons/FitVert_16x", Icon16x16));
		Set("CurveEd.CurveKey", new IMAGE_BRUSH("Common/Key", FVector2f(11.0f, 11.0f)));
		Set("CurveEd.CurveKeySelected", new IMAGE_BRUSH("Common/Key", FVector2f(11.0f, 11.0f), SelectionColor));
		Set("CurveEd.InfoFont", DEFAULT_FONT("Regular", 8));
		Set("CurveEd.LabelFont", DEFAULT_FONT("Bold", 10));
		Set("CurveEd.Tangent", new IMAGE_BRUSH("Common/Tangent", FVector2f(7.0f, 7.0f), FLinearColor(0.0f, 0.66f, 0.7f)));
		Set("CurveEd.TangentSelected", new IMAGE_BRUSH("Common/Tangent", FVector2f(7.0f, 7.0f), FLinearColor(1.0f, 1.0f, 0.0f)));
		Set("CurveEd.TangentColor", FLinearColor(0.0f, 0.66f, 0.7f));
		Set("CurveEd.TangentColorSelected", FLinearColor(1.0f, 1.0f, 0.0f));
	}
	
	// Scrub control buttons
	{
		Set("Animation.PlayControlsButton", FButtonStyle(Button)
			.SetNormal(FSlateNoResource())
			.SetDisabled(FSlateNoResource())
			.SetNormalPadding(FMargin(2.f, 2.f, 2.f, 2.f))
			.SetPressedPadding(FMargin(2.f, 2.f, 2.f, 2.f)));

		Set("Animation.Pause", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsPause", Icon20x20));
		Set("Animation.Forward", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsPlayForward", Icon20x20));
		Set("Animation.Forward_Step", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsToNext", Icon20x20));
		Set("Animation.Forward_End", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsToEnd", Icon20x20));
		Set("Animation.Backward", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsPlayReverse", Icon20x20));
		Set("Animation.Stop", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsStop", Icon20x20));
		Set("Animation.Backward_Step", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsToPrevious", Icon20x20));
		Set("Animation.Backward_End", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsToFront", Icon20x20));
		Set("Animation.Loop.Enabled", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsLooping", Icon20x20));
		Set("Animation.Loop.Disabled", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsNoLooping", Icon20x20));
		Set("Animation.Loop.SelectionRange", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsLoopingSelectionRange", Icon20x20));
		Set("Animation.Record", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsRecord", Icon20x20));
	}

	// Message Log
	{
		
		Set( "MessageLog.Action", new IMAGE_BRUSH( "Icons/icon_file_choosepackages_16px", Icon16x16) );
		Set( "MessageLog.Docs", new IMAGE_BRUSH( "Icons/icon_Docs_16x", Icon16x16) );
		Set( "MessageLog.Tutorial", new IMAGE_BRUSH( "Icons/icon_Blueprint_Enum_16x", Icon16x16 ) );
		Set( "MessageLog.Url", new IMAGE_BRUSH( "Icons/icon_world_16x", Icon16x16 ) );
		Set( "MessageLog.Fix", new IMAGE_BRUSH_SVG( "Starship/Common/wrench", Icon16x16 ) );

		Set( "MessageLog.TabIcon", new IMAGE_BRUSH_SVG( "Starship/Common/MessageLog", Icon16x16 ) );
		Set( "MessageLog.ListBorder", new BOX_BRUSH( "/Docking/AppTabContentArea", FMargin(4/16.0f) ) );
	}

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

	// Animation tools
	if (IncludeEditorSpecificStyles())
	{
		Set( "AnimEditor.RefreshButton", new IMAGE_BRUSH( "Old/AnimEditor/RefreshButton", Icon16x16 ) );
		Set( "AnimEditor.VisibleEye", new IMAGE_BRUSH( "Old/AnimEditor/RefreshButton", Icon16x16 ) );
		Set( "AnimEditor.InvisibleEye", new IMAGE_BRUSH( "Old/AnimEditor/RefreshButton", Icon16x16 ) );
		Set( "AnimEditor.FilterSearch", new IMAGE_BRUSH( "Old/FilterSearch", Icon16x16 ) );
		Set( "AnimEditor.FilterCancel", new IMAGE_BRUSH( "Old/FilterCancel", Icon16x16 ) );

		Set( "AnimEditor.NotifyGraphBackground", new IMAGE_BRUSH( "Old/AnimEditor/NotifyTrackBackground", Icon64x64, FLinearColor::White, ESlateBrushTileType::Both) );

		Set( "BlendSpace.SamplePoint", new IMAGE_BRUSH( "Old/AnimEditor/BlendSpace_Sample", Icon16x16 ) );
		Set( "BlendSpace.SamplePoint_Highlight", new IMAGE_BRUSH( "Old/AnimEditor/BlendSpace_Sample_Highlight", Icon16x16 ) );
		Set( "BlendSpace.SamplePoint_Invalid", new IMAGE_BRUSH( "Old/AnimEditor/BlendSpace_Sample_Invalid", Icon16x16 ) );
		Set( "BlendSpace.Graph", new IMAGE_BRUSH_SVG("Starship/Animation/BlendSpace", Icon16x16));
		Set( "BlendSpace.SampleGraph", new IMAGE_BRUSH_SVG("Starship/Animation/BlendSpace", Icon16x16));

		Set( "AnimEditor.EditPreviewParameters", new IMAGE_BRUSH( "Icons/icon_adjust_parameters_40x", Icon40x40) );		
		Set( "AnimEditor.EditPreviewParameters.Small", new IMAGE_BRUSH( "Icons/icon_adjust_parameters_40x", Icon20x20) );		
	}

#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	// Debugging tools 
	{
		Set("PerfTools.TabIcon", new IMAGE_BRUSH( "Icons/icon_tab_PerfTools_16x", Icon16x16 ) );
		Set("ClassViewer.TabIcon", new IMAGE_BRUSH_SVG( "Starship/Common/Class", Icon16x16 ) );
		Set("StructViewer.TabIcon", new IMAGE_BRUSH_SVG( "Starship/AssetIcons/UserDefinedStruct_16", Icon16x16 ) );
		Set("BlueprintDebugger.TabIcon", new IMAGE_BRUSH_SVG( "Starship/Common/BlueprintDebugger", Icon16x16 ) );
		Set("CollisionAnalyzer.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/Collision", Icon16x16));
		Set("ObjectBrowser.TabIcon", new IMAGE_BRUSH_SVG( "Starship/Common/ObjectsBrowser", Icon16x16 ) );
		Set("PixelInspector.TabIcon", new IMAGE_BRUSH_SVG( "Starship/Common/PixelInspector", Icon16x16 ) );
	}

	{
		Set("DeveloperTools.MenuIcon", new IMAGE_BRUSH_SVG( "Starship/Common/DeveloperTools", Icon16x16 ) );
		Set("UnrealInsights.MenuIcon", new IMAGE_BRUSH_SVG("Starship/Common/UnrealInsights", Icon16x16));
	}

	// Automation Tools Menu
	{
		Set("AutomationTools.MenuIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/AutomationTools", Icon16x16));
		Set("AutomationTools.TestAutomation", new IMAGE_BRUSH_SVG("Starship/Common/TestAutomation", Icon16x16));
	}

	// Translation Picker Menu
	{
		Set("TranslationPicker.MenuIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/TranslationPicker", Icon16x16));
	}

	// Session Browser tab
	{
		Set("SessionBrowser.Terminate.Font", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT( "Bold", 12))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 0.9f)));
	}

	// Session Frontend Window
	{
		Set("SessionFrontEnd.Tabs.Tools", new CORE_IMAGE_BRUSH("/Icons/icon_tab_Tools_16x", Icon16x16));
	}

	// Undo History Window
	{
		Set("UndoHistory.TabIcon", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/UndoHistory", Icon16x16 ) );
	}

	// InputBinding editor
	{
		Set( "InputBindingEditor.ContextFont", DEFAULT_FONT( "Bold", 9 ) );
		Set( "InputBindingEditor.ContextBorder", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, FLinearColor(0.5,0.5,0.5,1.0) ) );
		Set( "InputBindingEditor.SmallFont", DEFAULT_FONT( "Regular", 8 ) );

		Set( "InputBindingEditor.HeaderButton", FButtonStyle(NoBorder)
			.SetNormalPadding(FMargin( 1,1,2,2 ))
			.SetPressedPadding(FMargin( 2,2,2,2 )) );

		Set( "InputBindingEditor.HeaderButton.Disabled", new FSlateNoResource() );


		Set( "InputBindingEditor.Tab",  new IMAGE_BRUSH( "Icons/icon_tab_KeyBindings_16px", Icon16x16 ) );
		Set( "InputBindingEditor.AssetEditor",  new IMAGE_BRUSH( "Icons/icon_keyb_AssetEditor_16px", Icon16x16) );
		Set( "InputBindingEditor.AssetEditor",  new IMAGE_BRUSH( "Icons/icon_keyb_AssetEditor_16px", Icon16x16) );
		Set( "InputBindingEditor.GenericCommands",  new IMAGE_BRUSH( "Icons/icon_keyb_CommonCommands_16px", Icon16x16) );
		Set( "InputBindingEditor.FoliageEditMode",  new IMAGE_BRUSH( "Icons/icon_keyb_FoliageEditMode_16px", Icon16x16) );
		Set( "InputBindingEditor.LandscapeEditor",  new IMAGE_BRUSH( "Icons/icon_keyb_LandscapeEditor_16px", Icon16x16) );
		Set( "InputBindingEditor.LayersView",  new IMAGE_BRUSH( "Icons/icon_keyb_Layers_16px", Icon16x16) );
		Set( "InputBindingEditor.LevelEditor",  new IMAGE_BRUSH( "Icons/icon_keyb_LevelEditor_16px", Icon16x16) );
		Set( "InputBindingEditor.LevelViewport",  new IMAGE_BRUSH( "Icons/icon_keyb_LevelViewports_16px", Icon16x16) );
		Set( "InputBindingEditor.MainFrame",  new IMAGE_BRUSH( "Icons/icon_keyb_MainFrame_16px", Icon16x16) );
		Set( "InputBindingEditor.OutputLog",  new IMAGE_BRUSH( "Icons/icon_keyb_OutputLog_16px", Icon16x16) );
		Set( "InputBindingEditor.PlayWorld",  new IMAGE_BRUSH( "Icons/icon_keyb_PlayWorld_16px", Icon16x16) );
	}

	// Package restore
	{
		Set( "PackageRestore.FolderOpen", new IMAGE_BRUSH( "Icons/FolderOpen", FVector2f(18.f, 16.f) ) );
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

	// Package Dialog
	{
		Set( "PackageDialog.ListHeader", new BOX_BRUSH( "Old/SavePackages/ListHeader", 4.0f/32.0f ) );
		Set( "SavePackages.SCC_DlgCheckedOutOther", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/SCC_DlgCheckedOutOther", Icon16x16));
		Set( "SavePackages.SCC_DlgNotCurrent", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_DlgNotCurrent", Icon16x16));
		Set( "SavePackages.SCC_DlgReadOnly", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_DlgReadOnly", Icon16x16));

	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	// Layers General
	{
		Set( "Layer.Icon16x", new IMAGE_BRUSH( "Icons/layer_16x", Icon16x16 ) );
		Set( "Layer.VisibleIcon16x", new IMAGE_BRUSH( "Icons/icon_layer_visible", Icon16x16 ) );
		Set( "Layer.NotVisibleIcon16x", new IMAGE_BRUSH( "Icons/icon_layer_not_visible", Icon16x16 ) );
	}

	// Layer Stats
	{
		Set( "LayerStats.Item.ClearButton", new IMAGE_BRUSH( "Icons/Cross_12x", Icon12x12 ) );
	}

	// Layer Cloud
	{
		Set( "LayerCloud.Item.BorderImage", new BOX_BRUSH( "Common/RoundedSelection_16x", FMargin(4.0f/16.0f) ) );
		Set( "LayerCloud.Item.ClearButton", new IMAGE_BRUSH( "Icons/Cross_12x", Icon12x12 ) );
		Set( "LayerCloud.Item.LabelFont", DEFAULT_FONT( "Bold", 9 ) );
	}

	// Layer Browser
	{
		Set( "LayerBrowser.LayerContentsQuickbarBackground",  new CORE_BOX_BRUSH( "Common/DarkGroupBorder", 4.f/16.f ) );
		Set( "LayerBrowser.ExploreLayerContents",  new IMAGE_BRUSH( "Icons/ExploreLayerContents", Icon16x16 ) );
		Set( "LayerBrowser.ReturnToLayersList",  new IMAGE_BRUSH( "Icons/ReturnToLayersList", Icon16x16) );
		Set( "LayerBrowser.Actor.RemoveFromLayer", new IMAGE_BRUSH( "Icons/Cross_12x", Icon12x12 ) );

		Set( "LayerBrowserButton", FButtonStyle( Button )
			.SetNormal(FSlateNoResource())
			.SetHovered(BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ))
			.SetPressed(BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ))
		);

		Set( "LayerBrowserButton.LabelFont", DEFAULT_FONT( "Regular", 8 ) );
	}

	// DataLayer
	{
		Set("ClassIcon.DataLayer", new IMAGE_BRUSH_SVG("Icons/DataLayerEditor", Icon16x16));
		Set("DataLayer.Editor", new IMAGE_BRUSH_SVG("Icons/DataLayerEditor", Icon16x16));
		Set("DataLayer.Runtime", new IMAGE_BRUSH_SVG("Icons/DataLayerRuntime", Icon16x16));
		Set("DataLayer.External", new IMAGE_BRUSH_SVG("Icons/ExternalDataLayer", Icon16x16));
		Set("DataLayer.LoadedInEditor", new IMAGE_BRUSH_SVG("Icons/DataLayerLoadedInEditor", Icon16x16));
		Set("DataLayerBrowser.AddSelection", new IMAGE_BRUSH_SVG("Icons/DataLayerAddSelected", Icon16x16));
		Set("DataLayerBrowser.RemoveSelection", new IMAGE_BRUSH_SVG("Icons/DataLayerRemoveSelected", Icon16x16));
		Set("DataLayerBrowser.DataLayerContentsQuickbarBackground", new CORE_BOX_BRUSH("Common/DarkGroupBorder", 4.f / 16.f));
		Set("DataLayerBrowser.Actor.RemoveFromDataLayer", new IMAGE_BRUSH("Icons/Cross_12x", Icon12x12));
		Set("DataLayerBrowserButton", FButtonStyle(Button)
			.SetNormal(FSlateNoResource())
			.SetHovered(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
			.SetPressed(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
		);
		Set("DataLayerBrowser.LabelFont", DEFAULT_FONT("Regular", 9));
		Set("DataLayerBrowser.LabelFontBold", DEFAULT_FONT("BoldItalic", 10));
		Set("DataLayer.ColorIcon", new FSlateBoxBrush(NAME_None, 8.0f / 32.0f, FStyleColors::White));
	}

	// Zen
	{
		Set("Zen.Server", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Server", Icon16x16));
		Set("Zen.Server.Start", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Play", Icon16x16));
		Set("Zen.Server.Stop", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Stop", Icon16x16));
		Set("Zen.Server.Restart", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Update", Icon16x16));
		Set("Zen.Store", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Cylinder", Icon16x16));
		Set("Zen.WebBrowser", new IMAGE_BRUSH_SVG("Starship/Common/WebBrowser", Icon12x12));
		Set("Zen.FolderExplore", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/show-in-explorer", Icon12x12));
		Set("Zen.Clipboard", new IMAGE_BRUSH_SVG("Starship/Common/Clipboard", Icon16x16));	
		Set("Zen.Dashboard.Icon", new IMAGE_BRUSH_SVG("Starship/Zen/Zen_16", Icon16x16));
		Set("Zen.Dashboard.Application", new IMAGE_BRUSH_SVG("Starship/Zen/Zen_24", Icon24x24));

		Set("Zen.Status.UnavailableBG", new IMAGE_BRUSH_SVG("Starship/Zen/ZenStatusWithBadge_16", Icon16x16));
		Set("Zen.Status.Unavailable", new IMAGE_BRUSH_SVG("Starship/Zen/ZenStatusBadge", Icon16x16));

		Set("Zen.Status.IdleBG", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_IdleBG", Icon16x16));
		Set("Zen.Status.Idle", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_Idle", Icon16x16, EStyleColor::Success));

		Set("Zen.Status.BusyBG", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_UpDownBG", Icon16x16));
		Set("Zen.Status.Busy", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_UpDown", Icon16x16, FLinearColor(0.3f, 0.3f, 0.3f, 1.0f)));

		Set("Zen.Status.Uploading", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_Upload", Icon16x16, FLinearColor::Green));
		Set("Zen.Status.Downloading", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_Download", Icon16x16, FLinearColor(0.0f, 1.0f, 1.0f, 1.0f)));
		
		Set("Zen.Status.WarningBG", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_WarningBG", Icon16x16));
		Set("Zen.Status.Warning", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_Warning", Icon16x16, EStyleColor::Warning));
	}

	// Derived Data
	{
		Set("DerivedData.Cache.Settings", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_Cache_Settings", Icon16x16));
		Set("DerivedData.Cache.Statistics", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_Cache_Statistics", Icon16x16));
		Set("DerivedData.ResourceUsage", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_Resource_Usage", Icon16x16));
		
		Set("DerivedData.RemoteCache.Uploading", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_Upload", Icon16x16, FLinearColor::Green));
		Set("DerivedData.RemoteCache.Downloading", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_Download", Icon16x16, FLinearColor(0.0f, 1.0f, 1.0f, 1.0f)));

		Set("DerivedData.RemoteCache.BusyBG", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_UpDownBG", Icon16x16));
		Set("DerivedData.RemoteCache.Busy", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_UpDown", Icon16x16, FLinearColor(0.3f, 0.3f, 0.3f, 1.0f)));

		Set("DerivedData.RemoteCache.IdleBG", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_IdleBG", Icon16x16));
		Set("DerivedData.RemoteCache.Idle", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_Idle", Icon16x16, EStyleColor::Success));

		Set("DerivedData.RemoteCache.WarningBG", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_WarningBG", Icon16x16));
		Set("DerivedData.RemoteCache.Warning", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_Warning", Icon16x16, EStyleColor::Warning));

		Set("DerivedData.RemoteCache.UnavailableBG", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_Unavailable", Icon16x16));
		Set("DerivedData.RemoteCache.Unavailable", new IMAGE_BRUSH_SVG("Starship/DerivedData/DD_RemoteCache_Unavailable", Icon16x16));
	}

	// Editor Diagnostics
	{
		Set("EditorPerformance.Report.Panel", new IMAGE_BRUSH_SVG("Starship/Common/Diagnostics_16", Icon16x16));
		Set("EditorPerformance.Report.Minor", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16, EStyleColor::Warning));
		Set("EditorPerformance.Report.Major", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16, EStyleColor::AccentOrange));
		Set("EditorPerformance.Report.Critical", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle", Icon16x16, EStyleColor::Error));

		Set("EditorDiagnostics.StatusBar.Icon", new IMAGE_BRUSH_SVG("Starship/Common/Diagnostics_16", Icon16x16));
		Set("EditorDiagnostics.StatusBar.BadgeBG", new IMAGE_BRUSH_SVG("Starship/Common/DiagnosticsBadgeBG_16", Icon16x16));
		Set("EditorDiagnostics.StatusBar.WarningBadge", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-triangle-badge", Icon16x16, EStyleColor::Warning));

		Set("EditorDiagnostics.SetShowActivityMonitor", new IMAGE_BRUSH_SVG("Starship/Common/Scalability_16", Icon20x20));
		Set("EditorDiagnostics.SetShowStalls", new IMAGE_BRUSH_SVG("Starship/Common/Stall", Icon20x20));
		Set("EditorDiagnostics.SetShowCrashes", new IMAGE_BRUSH_SVG("Starship/Common/Debug", Icon20x20));

		Set("EditorDiagnostics.StatusBarButton", GetWidgetStyle<FButtonStyle>("StatusBar.StatusBarButton"));

		FButtonStyle StatusBarButtonAlert = GetWidgetStyle<FButtonStyle>("StatusBar.StatusBarButton");
		StatusBarButtonAlert
					   .SetNormal(FSlateColorBrush(FStyleColors::Error))
					   .SetNormalForeground(FStyleColors::ForegroundInverted)
					   .SetHovered(FSlateColorBrush(FStyleColors::Error))
					   .SetHoveredForeground(FStyleColors::ForegroundHover)
					   .SetPressed(FSlateColorBrush(FStyleColors::Error))
					   .SetPressedForeground(FStyleColors::ForegroundHover);

		Set("EditorDiagnostics.StatusBarButtonAlert", StatusBarButtonAlert);
	}

	// Scene Outliner
	{
		// Override icons
		
		// Background images for outliner override badges (first layer)
		Set("SceneOutliner.OverrideBase", new IMAGE_BRUSH_SVG("Starship/SceneOutliner/OutlinerOverrideBG", Icon16x16, FStyleColors::AccentBlue));
		Set("SceneOutliner.OverrideAddedBase", new IMAGE_BRUSH_SVG("Starship/SceneOutliner/OutlinerOverrideBG", Icon16x16, FStyleColors::AccentGreen));
		Set("SceneOutliner.OverrideRemovedBase", new IMAGE_BRUSH_SVG("Starship/SceneOutliner/OutlinerOverrideBG", Icon16x16, FStyleColors::AccentRed));
		Set("SceneOutliner.OverrideInsideBase", new IMAGE_BRUSH_SVG("Starship/SceneOutliner/OutlinerOverrideInsideBG", Icon16x16, FStyleColors::Input));

		// Foreground images for all override badges (second layer)
		Set("SceneOutliner.OverrideHere", new IMAGE_BRUSH_SVG("Starship/SceneOutliner/OutlinerOverrideHere", Icon16x16, FStyleColors::Input));
		Set("SceneOutliner.OverrideAdded", new IMAGE_BRUSH_SVG("Starship/SceneOutliner/OutlinerOverrideAdded", Icon16x16, FStyleColors::Input));
		Set("SceneOutliner.OverrideRemoved", new IMAGE_BRUSH_SVG("Starship/SceneOutliner/OutlinerOverrideRemoved", Icon16x16, FStyleColors::Input));
		Set("SceneOutliner.OverrideInside", new IMAGE_BRUSH_SVG("Starship/SceneOutliner/OutlinerOverrideInside", Icon16x16, FStyleColors::AccentBlue));
		Set("SceneOutliner.OverrideHereAndInside", new IMAGE_BRUSH_SVG("SceneOutliner/Common/OutlinerOverrideHereAndInside", Icon16x16, FStyleColors::AccentBlue));

		Set( "SceneOutliner.NewFolderIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-plus", Icon16x16 ) );
		Set( "SceneOutliner.FolderClosed",  new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", Icon16x16, FStyleColors::AccentFolder ) );
		Set( "SceneOutliner.FolderOpen",    new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-open", Icon16x16, FStyleColors::AccentFolder ) );
		Set( "SceneOutliner.CleanupActorFoldersIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-cleanup", Icon16x16));
		Set( "SceneOutliner.World", 		new CORE_IMAGE_BRUSH_SVG("Starship/Common/world", Icon16x16 ) );
		Set( "SceneOutliner.ChangedItemHighlight", new FSlateRoundedBoxBrush( FStyleColors::White, 1.0f) );

		const FTableRowStyle AlternatingTableRowStyle = GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow");
		
		Set( "SceneOutliner.TableViewRow", AlternatingTableRowStyle);
	}

	// Socket chooser
	{
		Set( "SocketChooser.TitleFont", DEFAULT_FONT( "Regular", 8 ) );
		Set( "SocketIcon.Bone", new IMAGE_BRUSH( "Old/bone", Icon16x16 ) );
		Set( "SocketIcon.Socket", new IMAGE_BRUSH( "Old/socket", Icon16x16 ) );
		Set( "SocketIcon.None", new IMAGE_BRUSH( "Old/Favorites_Disabled", Icon16x16 ) );
	}

	// Graph breadcrumb button
	{
		Set( "GraphBreadcrumbButton", FButtonStyle()
			.SetNormal        ( FSlateNoResource() )
			.SetPressed       ( BOX_BRUSH( "Common/Button_Pressed", 8.0f/32.0f, SelectionColor_Pressed ) )
			.SetHovered       ( BOX_BRUSH( "Common/Button_Hovered", 8.0f/32.0f, SelectionColor ) )
			.SetNormalPadding ( FMargin( 2.f,2.f,4.f,4.f) )
			.SetPressedPadding( FMargin( 3.f,3.f,3.f,3.f) )
		);

		Set( "GraphBreadcrumbButtonText", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 14 ) )
			.SetColorAndOpacity( FLinearColor(1.f,1.f,1.f,0.5f) )
			.SetShadowOffset( FVector2f::ZeroVector )
		);

		Set("GraphBreadcrumb.BrowseBack", new IMAGE_BRUSH_SVG( "Starship/Common/PreviousArrow", Icon20x20));
		Set("GraphBreadcrumb.BrowseForward", new IMAGE_BRUSH_SVG( "Starship/Common/NextArrow", Icon20x20));

		const FComboButtonStyle FastJumpComboBoxComboButton = FComboButtonStyle()
			.SetButtonStyle(GetWidgetStyle<FButtonStyle>("GraphBreadcrumbButton"));
		Set("GraphBreadcrumbFastJumpComboBoxStyle", FComboBoxStyle()
			.SetComboButtonStyle(FastJumpComboBoxComboButton));
	}

	// Graph bookmark button
	{

		Set("GraphBookmarkMenuImage.Button_Add", new IMAGE_BRUSH("Icons/PlusSymbol_12x", Icon12x12));
		Set("GraphBookmarkMenuText.EmptyListItem", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Fonts/Roboto-Italic", 9))
			.SetColorAndOpacity(FSlateColor::UseSubduedForeground()));
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	// Breadcrumb Trail
	{
		Set( "BreadcrumbButton", FButtonStyle()
			.SetNormal ( FSlateNoResource() )
			.SetPressed( BOX_BRUSH( "Common/Button_Pressed", 8.0f/32.0f, SelectionColor_Pressed ) )
			.SetHovered( BOX_BRUSH( "Common/Button_Pressed", 8.0f/32.0f, SelectionColor ) )
			);
		
	}

	// Notification List
	{
		Set( "NotificationList.Glow", new FSlateColorBrush( FColor(255, 255, 255, 255) ) );
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	// Asset editors (common)
	{
		Set( "AssetEditor.SaveAsset", new IMAGE_BRUSH_SVG("Starship/Common/SaveCurrent", Icon16x16));
		Set( "AssetEditor.SaveAssetAs", new IMAGE_BRUSH_SVG("Starship/Common/SaveCurrentAs", Icon16x16));

		Set( "AssetEditor.ReimportAsset", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/reimport", Icon40x40 ) );
		Set( "AssetEditor.ReimportAsset.Small", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/reimport", Icon20x20 ) );

		Set( "AssetEditor.ReadOnlyBorder", new FSlateRoundedBoxBrush(FStyleColors::Foreground, 10.0f));
		Set("AssetEditor.ReadOnlyOpenable", new IMAGE_BRUSH_SVG("Starship/AssetEditors/LockEye", Icon16x16));

		Set("AssetEditor.PreviewSceneSettings",
			new IMAGE_BRUSH_SVG("Starship/AssetEditors/PreviewSceneSettings_16", Icon16x16));
	}
		
	// Asset Thumbnail
	{
		Set( "AssetThumbnail.AssetBackground", new FSlateColorBrush(FStyleColors::Recessed));
		Set( "AssetThumbnail.ClassBackground", new IMAGE_BRUSH( "Common/ClassBackground_64x", Icon64x64, FLinearColor(0.75f, 0.75f, 0.75f, 1.0f) ) );
		Set( "AssetThumbnail.Font", DEFAULT_FONT( "Regular", 10 ) );
		Set( "AssetThumbnail.StatusOverflowFont", DEFAULT_FONT( "Bold", 9 ) );
		Set( "AssetThumbnail.StatusOverflowFontSmall", DEFAULT_FONT( "Regular", 5 ) );
		Set( "AssetThumbnail.FontSmall", DEFAULT_FONT( "Regular", 7 ) );
		Set( "AssetThumbnail.ColorAndOpacity", FLinearColor(1.75f, 1.75f, 1.75f, 1.f) );
		Set( "AssetThumbnail.HintFont", DEFAULT_FONT( "Regular", 8 ) );
		Set( "AssetThumbnail.HintFontSmall", DEFAULT_FONT( "Regular", 6 ) );
		Set( "AssetThumbnail.HintColorAndOpacity", FLinearColor(0.75f, 0.75f, 0.75f, 1.f) );
		Set( "AssetThumbnail.HintShadowOffset", FVector2f::UnitVector );
		Set( "AssetThumbnail.HintShadowColorAndOpacity", FLinearColor(0.f, 0.f, 0.f, 0.5f) );
		Set( "AssetThumbnail.HintBackground", new BOX_BRUSH( "Common/TableViewHeader", FMargin(8.0f/32.0f) ) );

		// AssetBorder
		// Round
		Set( "AssetThumbnail.AssetBorder", new FSlateRoundedBoxBrush(FStyleColors::Transparent, FVector4(4.0f, 4.0f, 4.0f, 4.0f), FStyleColors::Secondary, 2.f));
		Set( "AssetThumbnail.AssetBorderSmall", new FSlateRoundedBoxBrush(FStyleColors::Transparent, FVector4(4.0f, 4.0f, 4.0f, 4.0f), FStyleColors::Secondary, 1.f));
		// Sharp
		Set( "AssetThumbnail.AssetBorderSharp", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 0.f, FStyleColors::Secondary, 2.f));
		Set( "AssetThumbnail.AssetBorderSharpSmall", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 0.f, FStyleColors::Secondary, 1.f));

		FLinearColor OverlayColorAssetStatusOverlay = FStyleColors::Panel.GetSpecifiedColor();
		OverlayColorAssetStatusOverlay.A = 0.75f;
		Set("AssetThumbnail.AssetThumbnailStatusBar", new FSlateRoundedBoxBrush(OverlayColorAssetStatusOverlay, 2.f));
		FLinearColor OverlayColorAssetThumbnailOverlay = FStyleColors::Panel.GetSpecifiedColor();
		OverlayColorAssetThumbnailOverlay.A = 0.75f;
		Set("AssetThumbnail.AssetThumbnailBar", new FSlateRoundedBoxBrush(OverlayColorAssetThumbnailOverlay, 4.f));
		Set("AssetThumbnail.ToolTip.CommandBorder", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.f, FStyleColors::Foreground, 1.f));
		Set("AssetThumbnail.ToolTip.DarkCommandBorder", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.f, FStyleColors::Black, 1.f));
		Set("AssetThumbnail.ToolTip.ForegroundCommandBorder", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.f, FStyleColors::Foreground, 1.f));
		Set("AssetThumbnail.Tooltip.Border", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 0.f, COLOR("#484848FF"), 2.f));

		const FTextBlockStyle TooltipTextSubdued = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 8))
			.SetColorAndOpacity(FStyleColors::Foreground);
		Set("AssetThumbnail.Tooltip.MoreInfoText", TooltipTextSubdued);

		const FButtonStyle EditModePrimitives = FButtonStyle()
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.f, COLOR("#121212FF"), 1.f))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f, COLOR("#121212FF"), 1.f))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.f, COLOR("#121212FF"), 1.f));
		Set("AssetThumbnail.EditMode.Primitive", EditModePrimitives);

		const FButtonStyle ActionButton = FButtonStyle()
			.SetNormal(IMAGE_BRUSH_SVG("Starship/AssetActions/PlayButtonBackground", Icon32x32, FStyleColors::Secondary))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/AssetActions/PlayButtonBackground", Icon32x32, FStyleColors::Hover))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/AssetActions/PlayButtonBackground", Icon32x32, FStyleColors::Recessed));
		Set("AssetThumbnail.Action.Button", ActionButton);

	}

	// Open any asset dialog
	{
		Set( "SystemWideCommands.SummonOpenAssetDialog", new IMAGE_BRUSH_SVG( "Starship/Common/OpenAsset", Icon16x16 ) );
	
		Set( "GlobalAssetPicker.Background", new BOX_BRUSH( "Old/Menu_Background", FMargin(8.0f/64.0f) ) );
		Set( "GlobalAssetPicker.OutermostMargin", FMargin(4.f, 4.f, 4.f, 4.f) );

		Set( "GlobalAssetPicker.TitleFont", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 9 ) )
			.SetColorAndOpacity( FLinearColor::White )
			.SetShadowOffset( FVector2f::UnitVector )
			.SetShadowColorAndOpacity( FLinearColor::Black )
		);
	}


	// Main frame
	{
		Set( "MainFrame.AutoSaveImage", 	         new IMAGE_BRUSH_SVG( "Starship/Common/SaveCurrent", Icon16x16 ) );
		Set( "MainFrame.SaveAll",                    new IMAGE_BRUSH_SVG( "Starship/Common/SaveAll", Icon16x16 ) );
		Set( "MainFrame.ChoosePackagesToSave",       new IMAGE_BRUSH_SVG( "Starship/Common/SaveChoose", Icon16x16 ) );
		Set( "MainFrame.NewProject",                 new IMAGE_BRUSH_SVG( "Starship/Common/ProjectNew", Icon16x16 ) );
		Set( "MainFrame.OpenProject",                new IMAGE_BRUSH_SVG( "Starship/Common/ProjectOpen", Icon16x16 ) );
		Set( "MainFrame.AddCodeToProject",           new IMAGE_BRUSH_SVG( "Starship/Common/ProjectC++", Icon16x16 ) );
		Set( "MainFrame.Exit",                       new IMAGE_BRUSH_SVG( "Starship/Common/Exit", Icon16x16 ) );
		Set( "MainFrame.CookContent",                new IMAGE_BRUSH_SVG( "Starship/Common/CookContent", Icon16x16 ) );
		Set( "MainFrame.OpenVisualStudio",           new IMAGE_BRUSH_SVG( "Starship/Common/VisualStudio", Icon16x16 ) );
		Set( "MainFrame.RefreshVisualStudio",        new IMAGE_BRUSH_SVG( "Starship/Common/RefreshVisualStudio", Icon16x16 ) );
		Set( "MainFrame.OpenSourceCodeEditor",       new IMAGE_BRUSH_SVG( "Starship/Common/SourceCodeEditor", Icon16x16));
		Set( "MainFrame.RefreshSourceCodeEditor",    new IMAGE_BRUSH_SVG( "Starship/Common/RefreshSourceCodeEditor", Icon16x16));
		Set( "MainFrame.Project",                    new IMAGE_BRUSH_SVG( "Starship/Common/Project", Icon16x16 ) );
		Set( "MainFrame.PackageProject",             new IMAGE_BRUSH_SVG( "Starship/Common/ProjectPackage", Icon16x16 ) );
		Set( "MainFrame.RecentProjects",             new IMAGE_BRUSH_SVG( "Starship/Common/ProjectsRecent", Icon16x16 ) );
		Set( "MainFrame.RecentLevels",               new IMAGE_BRUSH_SVG( "Starship/Common/LevelRecent", Icon16x16 ) );
		Set( "MainFrame.FavoriteLevels",             new IMAGE_BRUSH_SVG( "Starship/Common/LevelFavorite", Icon16x16 ) );
		Set( "MainFrame.ZipUpProject", 		         new IMAGE_BRUSH_SVG( "Starship/Common/ZipProject", Icon16x16 ) );
		Set("MainFrame.OpenCurrentProjectDirectory", new IMAGE_BRUSH_SVG( "Starship/Common/BrowseContent", Icon16x16));

		Set( "MainFrame.ChooseFilesToSave",       new IMAGE_BRUSH_SVG( "Starship/Common/SaveChoose", Icon16x16 ) );
		Set( "MainFrame.ConnectToSourceControl",  new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/Status/RevisionControl", Icon16x16) );
		Set( "MainFrame.OpenMarketplace",			new IMAGE_BRUSH_SVG("Starship/MainToolbar/marketplace", Icon16x16));

		Set( "MainFrame.DebugTools.SmallFont", DEFAULT_FONT( "Regular", 8 ) );
		Set( "MainFrame.DebugTools.NormalFont", DEFAULT_FONT( "Regular", 9 ) );
		Set( "MainFrame.DebugTools.LabelFont", DEFAULT_FONT( "Regular", 8 ) );
	}

	// Editor preferences
	{
		Set("EditorPreferences.TabIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Preferences", Icon16x16));
		Set("SystemWideCommands.OpenEditorPreferences", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Preferences", Icon16x16));
		Set("SystemWideCommands.OpenKeyboardShortcuts", new IMAGE_BRUSH_SVG("Starship/Common/ViewportControls", Icon16x16));
	}

	// Project settings
	{
		Set("ProjectSettings.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/ProjectSettings", Icon16x16));
		Set("SystemWideCommands.OpenProjectSettings", new IMAGE_BRUSH_SVG("Starship/Common/ProjectSettings", Icon16x16));
	}

	// Main frame
	{
		Set("MainFrame.StatusInfoButton", FButtonStyle(Button)
			.SetNormal( IMAGE_BRUSH( "Icons/StatusInfo_16x", Icon16x16 ) )
			.SetHovered( IMAGE_BRUSH( "Icons/StatusInfo_16x", Icon16x16 ) )
			.SetPressed( IMAGE_BRUSH( "Icons/StatusInfo_16x", Icon16x16 ) )
			.SetNormalPadding(0)
			.SetPressedPadding(0)
		);
	}

	// CodeView selection detail view section
	{
		Set( "CodeView.ClassIcon", new IMAGE_BRUSH( "Icons/icon_class_16x", Icon16x16 ) );
		Set( "CodeView.FunctionIcon", new IMAGE_BRUSH( "Icons/icon_codeview_16x", Icon16x16 ) );
	}

	Set( "Editor.SearchBoxFont", DEFAULT_FONT( "Regular", 12) );
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	// About screen
	if (IncludeEditorSpecificStyles())
	{
		Set( "AboutScreen.Background", new IMAGE_BRUSH( "About/Background", FVector2f(688.f,317.f) ) );
		Set( "AboutScreen.UnrealLogo", new IMAGE_BRUSH_SVG( "About/UnrealLogo", Icon40x40 ) );
		Set( "AboutScreen.EpicGamesLogo", new IMAGE_BRUSH_SVG( "About/EpicGamesLogo", Icon40x40 ) );
		Set( "AboutScreen.TitleFont", DEFAULT_FONT( "Bold", 13) );
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR
	// Credits screen
	if (IncludeEditorSpecificStyles())
	{
		Set("Credits.Button", FButtonStyle(NoBorder)
			.SetNormal(FSlateNoResource())
			.SetPressed(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor_Pressed))
			.SetHovered(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, SelectionColor))
			);

		Set("Credits.Pause", new IMAGE_BRUSH("Icons/PauseCredits", Icon20x20));
		Set("Credits.Play", new IMAGE_BRUSH("Icons/PlayCredits", Icon20x20));

		FLinearColor EditorOrange = FLinearColor(0.728f, 0.364f, 0.003f);

		FTextBlockStyle CreditsNormal = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 16))
			.SetShadowOffset(FVector2f::UnitVector);

		Set("Credits.Normal", CreditsNormal);

		Set("Credits.Strong", FTextBlockStyle(CreditsNormal)
			.SetFont(DEFAULT_FONT("Bold", 16))
			.SetShadowOffset(FVector2f::UnitVector));

		Set("Credits.H1", FTextBlockStyle(CreditsNormal)
			.SetColorAndOpacity(EditorOrange)
			.SetFont(DEFAULT_FONT("Bold", 36))
			.SetShadowOffset(FVector2f::UnitVector));

		Set("Credits.H2", FTextBlockStyle(CreditsNormal)
			.SetColorAndOpacity(EditorOrange)
			.SetFont(DEFAULT_FONT("Bold", 30))
			.SetShadowOffset(FVector2f::UnitVector));

		Set("Credits.H3", FTextBlockStyle(CreditsNormal)
			.SetFont(DEFAULT_FONT("Bold", 24))
			.SetShadowOffset(FVector2f::UnitVector));

		Set("Credits.H4", FTextBlockStyle(CreditsNormal)
			.SetFont(DEFAULT_FONT("Bold", 18))
			.SetShadowOffset(FVector2f::UnitVector));

		Set("Credits.H5", FTextBlockStyle(CreditsNormal)
			.SetFont(DEFAULT_FONT("Bold", 12))
			.SetShadowOffset(FVector2f::UnitVector));

		Set("Credits.H6", FTextBlockStyle(CreditsNormal)
			.SetFont(DEFAULT_FONT("Bold", 6))
			.SetShadowOffset(FVector2f::UnitVector));

		FTextBlockStyle LinkText = FTextBlockStyle(NormalText)
			.SetColorAndOpacity(EditorOrange)
			.SetShadowOffset(FVector2f::UnitVector);
		FButtonStyle HoverOnlyHyperlinkButton = FButtonStyle()
			.SetNormal(FSlateNoResource())
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0.f, 0.f, 0.f, 3.f / 16.0f)));
		FHyperlinkStyle HoverOnlyHyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(HoverOnlyHyperlinkButton)
			.SetTextStyle(LinkText)
			.SetPadding(FMargin(0.0f));

		Set("Credits.Hyperlink", HoverOnlyHyperlink);
	}
#endif // WITH_EDITOR

	// Hardware target settings
#if WITH_EDITOR
	{
		FLinearColor EditorOrange = FLinearColor(0.728f, 0.364f, 0.003f);

		FTextBlockStyle TargetSettingsNormal = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 8));

		Set("HardwareTargets.Normal", TargetSettingsNormal);

		Set("HardwareTargets.Strong", FTextBlockStyle(TargetSettingsNormal)
			.SetFont(DEFAULT_FONT("Bold", 8))
			.SetColorAndOpacity(EditorOrange)
			.SetShadowOffset(FVector2f::UnitVector));
	}
#endif

	// New Level Dialog
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	{
		Set( "NewLevelDialog.Blank", new IMAGE_BRUSH( "NewLevels/NewLevelBlank", FVector2f(256.f,256.f) ) );
		Set( "NewLevelDialog.BlankWP", new IMAGE_BRUSH("NewLevels/NewLevelBlankWP", FVector2f(256.f, 256.f) ) );
	}

	// Build and Submit
	{
		Set( "BuildAndSubmit.NormalFont", DEFAULT_FONT( "Regular", 8 ) );
		Set( "BuildAndSubmit.SmallFont", DEFAULT_FONT( "Regular", 7 ) );
	}

	// Foliage Edit Mode
	if (IncludeEditorSpecificStyles())
	{	
		FLinearColor DimBackground = FLinearColor(FColor(64, 64, 64));
		FLinearColor DimBackgroundHover = FLinearColor(FColor(50, 50, 50));
		FLinearColor DarkBackground = FLinearColor(FColor(42, 42, 42));

		FToolBarStyle FoliageEditToolBar = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("ToolBar");

		FoliageEditToolBar.SetButtonPadding(FMargin(0.f));
		FoliageEditToolBar.SetComboButtonPadding(FMargin(4.0f));
		FoliageEditToolBar.SetCheckBoxPadding(FMargin(10.0f, 6.f));
		FoliageEditToolBar.SetSeparatorPadding(1.0f);
		FoliageEditToolBar.SetToggleButtonStyle(
			FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(BOX_BRUSH("Common/Selection", 8.0f / 32.0f, DimBackground))
			.SetUncheckedPressedImage(BOX_BRUSH("PlacementMode/TabActive", 8.0f / 32.0f))
			.SetUncheckedHoveredImage(BOX_BRUSH("Common/Selection", 8.0f / 32.0f, DimBackgroundHover))
			.SetCheckedImage(BOX_BRUSH("PlacementMode/TabActive", 8.0f / 32.0f))
			.SetCheckedHoveredImage(BOX_BRUSH("PlacementMode/TabActive", 8.0f / 32.0f))
			.SetCheckedPressedImage(BOX_BRUSH("PlacementMode/TabActive", 8.0f / 32.0f))
			.SetPadding(0));

		Set("FoliageEditToolBar", FoliageEditToolBar);

		Set("FoliageEditMode.SetSelect",                 new IMAGE_BRUSH("Icons/GeneralTools/Select_40x", Icon20x20));
		Set("FoliageEditMode.SetSelect.Small",           new IMAGE_BRUSH("Icons/GeneralTools/Select_40x", Icon20x20));
		Set("FoliageEditMode.SelectAll",                 new IMAGE_BRUSH("Icons/GeneralTools/SelectAll_40x", Icon20x20));
		Set("FoliageEditMode.SelectAll.Small",           new IMAGE_BRUSH("Icons/GeneralTools/SelectAll_40x", Icon20x20));
		Set("FoliageEditMode.DeselectAll",               new IMAGE_BRUSH("Icons/GeneralTools/Deselect_40x", Icon20x20));
		Set("FoliageEditMode.DeselectAll.Small",         new IMAGE_BRUSH("Icons/GeneralTools/Deselect_40x", Icon20x20));
		Set("FoliageEditMode.SelectInvalid",             new IMAGE_BRUSH("Icons/GeneralTools/SelectInvalid_40x", Icon20x20));
		Set("FoliageEditMode.SelectInvalid.Small",       new IMAGE_BRUSH("Icons/GeneralTools/SelectInvalid_40x", Icon20x20));
		Set("FoliageEditMode.SetLassoSelect",            new IMAGE_BRUSH("Icons/GeneralTools/Lasso_40x", Icon20x20));
		Set("FoliageEditMode.SetLassoSelect.Small",      new IMAGE_BRUSH("Icons/GeneralTools/Lasso_40x", Icon20x20));
		Set("FoliageEditMode.Foliage",                   new IMAGE_BRUSH("Icons/GeneralTools/Foliage_40x", Icon20x20));
		Set("FoliageEditMode.Foliage.Small",             new IMAGE_BRUSH("Icons/GeneralTools/Foliage_40x", Icon20x20));
		Set("FoliageEditMode.SetPaint",                  new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("FoliageEditMode.SetPaint.Small",            new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("FoliageEditMode.SetReapplySettings",        new IMAGE_BRUSH("Icons/GeneralTools/Repaint_40x", Icon20x20));
		Set("FoliageEditMode.SetReapplySettings.Small",  new IMAGE_BRUSH("Icons/GeneralTools/Repaint_40x", Icon20x20));
		Set("FoliageEditMode.SetPaintBucket",            new IMAGE_BRUSH("Icons/GeneralTools/PaintBucket_40x", Icon20x20));
		Set("FoliageEditMode.SetPaintBucket.Small",      new IMAGE_BRUSH("Icons/GeneralTools/PaintBucket_40x", Icon20x20));
		Set("FoliageEditMode.Remove",                    new IMAGE_BRUSH("Icons/GeneralTools/Delete_40x", Icon20x20));
		Set("FoliageEditMode.Remove.Small",              new IMAGE_BRUSH("Icons/GeneralTools/Delete_40x", Icon20x20));
		Set("FoliageEditMode.Erase",                     new IMAGE_BRUSH("Icons/GeneralTools/Erase_40x", Icon20x20));
		Set("FoliageEditMode.Erase.Small",               new IMAGE_BRUSH("Icons/GeneralTools/Erase_40x", Icon20x20));
		Set("FoliageEditMode.Filter",                    new IMAGE_BRUSH("Icons/GeneralTools/Filter_40x", Icon20x20));
		Set("FoliageEditMode.Filter.Small",              new IMAGE_BRUSH("Icons/GeneralTools/Filter_40x", Icon20x20));
		Set("FoliageEditMode.Settings",                  new IMAGE_BRUSH("Icons/GeneralTools/Settings_40x", Icon20x20));
		Set("FoliageEditMode.Settings.Small",            new IMAGE_BRUSH("Icons/GeneralTools/Settings_40x", Icon20x20));
		Set("FoliageEditMode.MoveToCurrentLevel",        new IMAGE_BRUSH("Icons/GeneralTools/MoveToLevel_40x", Icon20x20));
		Set("FoliageEditMode.MoveToCurrentLevel.Small",  new IMAGE_BRUSH("Icons/GeneralTools/MoveToLevel_40x", Icon20x20));
		Set("FoliageEditMode.MoveToActorEditorContext", new IMAGE_BRUSH("Icons/GeneralTools/MoveToActorEditorContext_40x", Icon20x20));
		Set("FoliageEditMode.MoveToActorEditorContext.Small", new IMAGE_BRUSH("Icons/GeneralTools/MoveToActorEditorContext_40x", Icon20x20));

		Set( "FoliageEditMode.SetNoSettings", new IMAGE_BRUSH( "Icons/FoliageEditMode/icon_FoliageEdMode_NoSettings_20x", Icon20x20 ) );
		Set( "FoliageEditMode.SetPaintSettings", new IMAGE_BRUSH( "Icons/FoliageEditMode/icon_FoliageEdMode_PaintingSettings_20x", Icon20x20 ) );
		Set( "FoliageEditMode.SetClusterSettings", new IMAGE_BRUSH( "Icons/FoliageEditMode/icon_FoliageEdMode_ClusterSettings_20x", Icon20x20 ) );
		Set( "FoliageEditMode.SetNoSettings.Small", new IMAGE_BRUSH( "Icons/FoliageEditMode/icon_FoliageEdMode_NoSettings_20x", Icon20x20 ) );
		Set( "FoliageEditMode.SetPaintSettings.Small", new IMAGE_BRUSH( "Icons/FoliageEditMode/icon_FoliageEdMode_PaintingSettings_20x", Icon20x20 ) );
		Set( "FoliageEditMode.SetClusterSettings.Small", new IMAGE_BRUSH( "Icons/FoliageEditMode/icon_FoliageEdMode_ClusterSettings_20x", Icon20x20 ) );

		Set( "FoliageEditMode.OpenSettings", new IMAGE_BRUSH( "Icons/FoliageEditMode/icon_FoliageEditMode_LoadSettings_20px", Icon20x20 ) );
		Set( "FoliageEditMode.SaveSettings", new IMAGE_BRUSH( "Icons/FoliageEditMode/icon_FoliageEditMode_SaveSettings_20px", Icon20x20 ) );
		Set( "FoliageEditMode.DeleteItem", new IMAGE_BRUSH( "Icons/FoliageEditMode/icon_FoliageEditMode_RemoveSettings_20x", Icon20x20 ) );
		Set( "FoliageEditMode.SelectionBackground", new IMAGE_BRUSH( "Icons/FoliageEditMode/FoliageEditMode_SelectionBackground", Icon32x32 ) );
		Set( "FoliageEditMode.ItemBackground", new IMAGE_BRUSH( "Icons/FoliageEditMode/FoliageEditMode_Background", Icon64x64 ) );
		Set( "FoliageEditMode.BubbleBorder", new BOX_BRUSH( "Icons/FoliageEditMode/FoliageEditMode_BubbleBorder", FMargin(8.f/32.0f) ) );

		Set( "FoliageEditMode.TreeView.ScrollBorder", FScrollBorderStyle()
			.SetTopShadowBrush(FSlateNoResource())
			.SetBottomShadowBrush(BOX_BRUSH("Common/ScrollBorderShadowBottom", FVector2f(16.f, 8.f), FMargin(0.5f, 0.f, 0.5f, 1.f)))
			);

		Set("FoliageEditMode.Splitter", FSplitterStyle()
			.SetHandleNormalBrush(IMAGE_BRUSH("Common/SplitterHandleHighlight", Icon8x8, FLinearColor(.2f, .2f, .2f, 1.f)))
			.SetHandleHighlightBrush(IMAGE_BRUSH("Common/SplitterHandleHighlight", Icon8x8, FLinearColor::White))
			);

		Set("FoliageEditMode.ActiveToolName.Text", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 11))
			.SetShadowOffset(FVector2f::UnitVector)
			);

		Set("FoliageEditMode.AddFoliageType.Text", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 10))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.9f)));


		FLinearColor InputA = FStyleColors::Input.GetSpecifiedColor();
		InputA .A = .60f;

		FLinearColor InputB = FStyleColors::Input.GetSpecifiedColor();
		InputA .A = .70f;

		Set("FoliageEditMode.FloatingButton", FButtonStyle()
			.SetNormal(FSlateRoundedBoxBrush(InputA, 2.f))
			.SetHovered(FSlateRoundedBoxBrush(InputB, 2.f))
			.SetPressed(FSlateRoundedBoxBrush(InputB, 2.f))
			.SetNormalForeground(FStyleColors::Foreground)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::White25)
			.SetNormalPadding(FMargin(4.f))
			.SetPressedPadding(FMargin(4.f))
		 );

	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)


#if WITH_EDITOR
	// Surface Props
	{
		Set( "SurfaceDetails.PanUPositive", new IMAGE_BRUSH( "Icons/icon_PanRight", Icon16x16 ) );
		Set( "SurfaceDetails.PanUNegative", new IMAGE_BRUSH( "Icons/icon_PanLeft", Icon16x16 ) );

		Set( "SurfaceDetails.PanVPositive", new IMAGE_BRUSH( "Icons/icon_PanUp", Icon16x16 ) );
		Set( "SurfaceDetails.PanVNegative", new IMAGE_BRUSH( "Icons/icon_PanDown", Icon16x16 ) );

		
		Set( "SurfaceDetails.ClockwiseRotation", new IMAGE_BRUSH( "Icons/icon_ClockwiseRotation_16x", Icon16x16 ) );
		Set( "SurfaceDetails.AntiClockwiseRotation", new IMAGE_BRUSH( "Icons/icon_AntiClockwiseRotation_16x", Icon16x16 ) );
	}

	// GameProjectDialog
	if (IncludeEditorSpecificStyles())
	{
		Set( "GameProjectDialog.BlankProjectThumbnail", new IMAGE_BRUSH( "GameProjectDialog/blank_project_thumbnail", Icon128x128 ) );
		Set( "GameProjectDialog.BlankProjectPreview", new IMAGE_BRUSH( "GameProjectDialog/blank_project_preview", FVector2f(400.f, 200.f) ) );

	}

	// NewClassDialog
	if (IncludeEditorSpecificStyles())
	{
		Set( "NewClassDialog.ErrorLabelCloseButton", new IMAGE_BRUSH( "Icons/Cross_12x", Icon12x12 ) );

		Set( "NewClassDialog.ParentClassListView.TableRow", FTableRowStyle()
			.SetEvenRowBackgroundBrush( FSlateNoResource() )
			.SetEvenRowBackgroundHoveredBrush(FSlateRoundedBoxBrush(FStyleColors::Panel, 4.0f))
			.SetOddRowBackgroundBrush( FSlateNoResource() )
			.SetOddRowBackgroundHoveredBrush(FSlateRoundedBoxBrush(FStyleColors::Panel, 4.0f))
			.SetSelectorFocusedBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Select, 1.0f))
			.SetActiveBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Select, 1.0f))
			.SetActiveHoveredBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Select, 1.0f))
			.SetInactiveBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::SelectInactive, 1.0f))
			.SetInactiveHoveredBrush(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::SelectInactive, 1.0f))
			.SetTextColor( FStyleColors::Foreground )
			.SetSelectedTextColor(FStyleColors::Foreground)
			);

	}

	// Package Migration
	{
		Set( "PackageMigration.DialogTitle", FTextBlockStyle( NormalText )
			.SetFont( DEFAULT_FONT( "Regular", 12 ) )
		);
	}

	// Hardware Targeting
	{
		Set( "HardwareTargeting.MobilePlatform", new IMAGE_BRUSH( "/Icons/HardwareTargeting/Mobile", Icon64x64 ) );
		Set( "HardwareTargeting.DesktopPlatform", new IMAGE_BRUSH( "/Icons/HardwareTargeting/Desktop", Icon64x64) );
		Set( "HardwareTargeting.HardwareUnspecified", new IMAGE_BRUSH( "/Icons/HardwareTargeting/HardwareUnspecified", Icon64x64) );

		Set( "HardwareTargeting.MaximumQuality", new IMAGE_BRUSH( "/Icons/HardwareTargeting/MaximumQuality", Icon64x64) );
		Set( "HardwareTargeting.ScalableQuality", new IMAGE_BRUSH( "/Icons/HardwareTargeting/ScalableQuality", Icon64x64) );
		Set( "HardwareTargeting.GraphicsUnspecified", new IMAGE_BRUSH( "/Icons/HardwareTargeting/GraphicsUnspecified", Icon64x64) );
	}
	
	// SettingsEditor
	{
		Set( "SettingsEditor.ListBorder", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f) );

		constexpr float PaddingAmount = 2.0f;
		constexpr float ExtraTopPadding = 12.f;
		constexpr float ExtraBottomPadding = 8.f;
		constexpr float AllPluginsExtraTopPadding = 9.f;
		constexpr float AllPluginsExtraBottomPadding = 7.f;

		Set( "CategoryTreeItem.PaddingAmount", PaddingAmount);
		Set( "CategoryTreeItem.Root.BackgroundBrush", new FSlateNoResource );
		Set( "CategoryTreeItem.Root.BackgroundPadding", FMargin(PaddingAmount, PaddingAmount + ExtraTopPadding, PaddingAmount, PaddingAmount + ExtraBottomPadding) );
		Set( "CategoryTreeItem.Root.AllBackgroundPadding", FMargin(PaddingAmount, PaddingAmount + AllPluginsExtraTopPadding, PaddingAmount, PaddingAmount + AllPluginsExtraBottomPadding) );

		const FTextBlockStyle ButtonStyleText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("PrimaryButtonText");
		FTextBlockStyle Text = FTextBlockStyle(ButtonStyleText);
		Text.ColorAndOpacity = FStyleColors::Foreground;
		Text.TransformPolicy = ETextTransformPolicy::ToUpper;
		Set( "CategoryTreeItem.Root.Text", Text );
		Set( "CategoryTreeItem.BackgroundBrush", new FSlateNoResource );
		Set( "CategoryTreeItem.BackgroundPadding", FMargin(PaddingAmount) );

		const FTextBlockStyle NormalStyleText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
		FTextBlockStyle CategoryText = FTextBlockStyle(NormalStyleText);
		CategoryText.ColorAndOpacity = FStyleColors::Foreground;
		Set( "CategoryTreeItem.Text", CategoryText );
	}

#endif // WITH_EDITOR

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	// Mode ToolPalette 
	{

		FToolBarStyle PaletteToolBarStyle = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		FTextBlockStyle PaletteToolbarLabelStyle = FTextBlockStyle(GetParentStyle()->GetWidgetStyle<FTextBlockStyle>("SmallText"));
		PaletteToolbarLabelStyle.SetOverflowPolicy(ETextOverflowPolicy::Ellipsis);

		PaletteToolBarStyle.SetLabelStyle(PaletteToolbarLabelStyle);
		
		PaletteToolBarStyle.SetBackground(FSlateColorBrush(FStyleColors::Recessed));

		PaletteToolBarStyle.SetLabelPadding(FMargin(0.0f, 1.0f, 0.0f, 0.0f));

		PaletteToolBarStyle.SetButtonPadding(       FMargin(0.0f, 0.0f));
		PaletteToolBarStyle.SetCheckBoxPadding(     FMargin(0.0f, 0.0f));
		PaletteToolBarStyle.SetComboButtonPadding(  FMargin(0.0f, 0.0f));
		PaletteToolBarStyle.SetIndentedBlockPadding(FMargin(0.0f, 0.0f));
		PaletteToolBarStyle.SetBlockPadding(        FMargin(0.0f, 0.0f));
		PaletteToolBarStyle.ToggleButton.SetPadding(FMargin(0.0f, 6.0f));
		PaletteToolBarStyle.ButtonStyle.SetNormalPadding(FMargin(2.0f, 6.0f));
		PaletteToolBarStyle.ButtonStyle.SetPressedPadding(FMargin(2.0f, 6.0f));

		Set("PaletteToolBar.Tab",  FCheckBoxStyle()
			.SetCheckBoxType(            ESlateCheckBoxType::ToggleButton)

			.SetCheckedImage(            FSlateRoundedBoxBrush(FStyleColors::Primary, 2.0f))
			.SetCheckedHoveredImage(     FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 2.0f))
			.SetCheckedPressedImage(     FSlateRoundedBoxBrush(FStyleColors::Dropdown, 2.0f))

			.SetUncheckedImage(          FSlateRoundedBoxBrush(FStyleColors::Secondary, 2.0f))
			.SetUncheckedHoveredImage(   FSlateRoundedBoxBrush(FStyleColors::Hover, 2.0f))
			.SetUncheckedPressedImage(   FSlateRoundedBoxBrush(FStyleColors::Secondary, 2.0f))

			.SetForegroundColor(         FStyleColors::Foreground)
			.SetHoveredForegroundColor(  FStyleColors::ForegroundHover)
			.SetPressedForegroundColor(  FStyleColors::ForegroundHover)
			.SetCheckedForegroundColor(  FStyleColors::ForegroundHover)
			.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetPadding(FMargin(2.f, 6.f))
		);

		Set("PaletteToolBar.MaxUniformToolbarSize", 48.f);
		Set("PaletteToolBar.MinUniformToolbarSize", 48.f);

		Set("PaletteToolBar.ExpandableAreaHeader", new FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(4.0, 4.0, 0.0, 0.0)));
		Set("PaletteToolBar.ExpandableAreaBody", new FSlateRoundedBoxBrush(FStyleColors::Recessed, FVector4(0.0, 0.0, 4.0, 4.0)));

	
		Set("PaletteToolBar", PaletteToolBarStyle);

		Set("EditorModesPanel.CategoryFontStyle", DEFAULT_FONT( "Bold", 10 ));
		Set("EditorModesPanel.ToolDescriptionFont", DEFAULT_FONT("Italic", 10));

	}

	{

		
		FToolBarStyle SlimPaletteToolBarStyle = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		FTextBlockStyle SlimPaletteToolbarLabelStyle = FTextBlockStyle(GetParentStyle()->GetWidgetStyle<FTextBlockStyle>("NormalText"));
		SlimPaletteToolbarLabelStyle.SetOverflowPolicy(ETextOverflowPolicy::Ellipsis);
		SlimPaletteToolBarStyle.SetLabelStyle(SlimPaletteToolbarLabelStyle);
		SlimPaletteToolBarStyle.SetBackgroundPadding(FMargin(2.0f, 4.0f, 6.0f, 8.0f));
		SlimPaletteToolBarStyle.SetBackground(FSlateColorBrush(FStyleColors::Panel));
		SlimPaletteToolBarStyle.SetLabelPadding(FMargin(0.0f, 0.0f, 4.0f, 0.0f));
		SlimPaletteToolBarStyle.SetIconSize(Icon20x20);

		SlimPaletteToolBarStyle.SetButtonPadding(       FMargin(4.0f, 4.0f, 0.0f, 0.0f));
		SlimPaletteToolBarStyle.SetCheckBoxPadding(     FMargin(4.0f, 4.0f, 0.0f, 0.0f));
		SlimPaletteToolBarStyle.SetIconPadding( FMargin(8.f, 4.f, 8.f, 4.f) );
		SlimPaletteToolBarStyle.SetComboButtonPadding(  FMargin(0.0f, 0.0f));
		SlimPaletteToolBarStyle.SetIndentedBlockPadding(FMargin(0.0f, 0.0f));
		SlimPaletteToolBarStyle.SetBlockPadding(        FMargin(0.0f, 0.0f));
		SlimPaletteToolBarStyle.ToggleButton.SetPadding(FMargin(0.0f, 0.0f));
		SlimPaletteToolBarStyle.ButtonStyle.SetNormalPadding(FMargin(0.0f, 0.0f));
		SlimPaletteToolBarStyle.ButtonStyle.SetPressedPadding(FMargin(0.0f, 0.0f));

		SlimPaletteToolBarStyle.ButtonStyle.Normal = FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f);
		SlimPaletteToolBarStyle.ButtonStyle.Hovered = FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f);
		SlimPaletteToolBarStyle.ButtonStyle.Pressed = FSlateRoundedBoxBrush(FStyleColors::Hover, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f);
		SlimPaletteToolBarStyle.ButtonStyle.HoveredForeground = FStyleColors::ForegroundHover;
		SlimPaletteToolBarStyle.ButtonStyle.PressedForeground = FStyleColors::ForegroundHover;
		SlimPaletteToolBarStyle.SetUniformBlockHeight(33.f);
		SlimPaletteToolBarStyle.SetUniformBlockWidth(150.f);
		SlimPaletteToolBarStyle.SetNumColumns(2);
		
		FCheckBoxStyle CheckBoxStyle = FCheckBoxStyle(SlimPaletteToolBarStyle.ToggleButton)
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f))
			.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, 4.f, FLinearColor(0.f, 0.f, 0.f, .8f), 0.5f))
			.SetForegroundColor(FStyleColors::Foreground)
			.SetPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetPadding(FMargin(0.f, 0.f, 0.f, 0.f));

		Set("SlimPaletteToolBarStyle.ToggleButton", CheckBoxStyle);
		SlimPaletteToolBarStyle.SetToggleButtonStyle(CheckBoxStyle);
		Set("SlimPaletteToolBar", SlimPaletteToolBarStyle);
		
	}


	{
		// FCategoryDrivenContentBuilder vertical toolbar style
		FToolBarStyle CategoryDrivenContentBuilderToolBarStyle =  GetWidgetStyle<FToolBarStyle>("FVerticalToolBar");
		CategoryDrivenContentBuilderToolBarStyle.SetIconPadding( FMargin(0.f, 9.f, 0.f, 4.f) );
		CategoryDrivenContentBuilderToolBarStyle.SetBackgroundPadding( FMargin( 4.f ) );
		CategoryDrivenContentBuilderToolBarStyle.SetIconPaddingWithVisibleLabel( FMargin(0.f, 9.f, 0.f, 0.f) );
		CategoryDrivenContentBuilderToolBarStyle.SetLabelPadding( FMargin(4.f, 5.f, 4.f, 9.f) );
		CategoryDrivenContentBuilderToolBarStyle.SetButtonContentMaxWidth( 56.0f );
		FTextBlockStyle LabelTextStyle = FTextBlockStyle(NormalText)
		                                 .SetOverflowPolicy(ETextOverflowPolicy::Ellipsis)
		                                 .SetFont(DEFAULT_FONT("roboto", FCoreStyle::SmallTextSize));
		CategoryDrivenContentBuilderToolBarStyle.SetLabelStyle( LabelTextStyle );
		CategoryDrivenContentBuilderToolBarStyle.SetButtonPadding( FMargin( 0.f, 2.f, 0.f, 2.f));
		CategoryDrivenContentBuilderToolBarStyle.SetShowLabels( true );
		Set("CategoryDrivenContentBuilderToolbarWithLabels", CategoryDrivenContentBuilderToolBarStyle);

		CategoryDrivenContentBuilderToolBarStyle.SetIconPadding( FMargin(8.f) );
		CategoryDrivenContentBuilderToolBarStyle.SetShowLabels( false );
		CategoryDrivenContentBuilderToolBarStyle.SetButtonContentMaxWidth( 36.0f );
		Set("CategoryDrivenContentBuilderToolbarWithoutLabels", CategoryDrivenContentBuilderToolBarStyle);
	}
	
	// Vertical ToolPalette 
	{
		FToolBarStyle VerticalToolBarStyle = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		FTextBlockStyle VerticalToolBarLabelStyle = FTextBlockStyle(GetParentStyle()->GetWidgetStyle<FTextBlockStyle>("SmallText"));
		VerticalToolBarLabelStyle.SetOverflowPolicy(ETextOverflowPolicy::Ellipsis);

		VerticalToolBarStyle.SetLabelStyle(VerticalToolBarLabelStyle);
		
		VerticalToolBarStyle.SetBackground(FSlateColorBrush(FStyleColors::Recessed));

		VerticalToolBarStyle.SetLabelPadding(FMargin(0.0f, 1.0f, 0.0f, 0.0f));

		VerticalToolBarStyle.SetButtonPadding(       FMargin(0.0f, 0.0f));
		VerticalToolBarStyle.SetCheckBoxPadding(     FMargin(0.0f, 0.0f));
		VerticalToolBarStyle.SetComboButtonPadding(  FMargin(0.0f, 0.0f));
		VerticalToolBarStyle.SetIndentedBlockPadding(FMargin(0.0f, 0.0f));
		VerticalToolBarStyle.SetBlockPadding(        FMargin(0.0f, 0.0f));
		VerticalToolBarStyle.SetBackgroundPadding(   FMargin(4.0f, 2.0f));
		VerticalToolBarStyle.ToggleButton.SetPadding(FMargin(0.0f, 6.0f));
		VerticalToolBarStyle.ButtonStyle.SetNormalPadding(FMargin(2.0f, 6.0f));
		VerticalToolBarStyle.ButtonStyle.SetPressedPadding(FMargin(2.0f, 6.0f));

		Set( "VerticalToolBar.Tab",  FCheckBoxStyle()
			.SetCheckBoxType(            ESlateCheckBoxType::ToggleButton)

			.SetCheckedImage(            FSlateRoundedBoxBrush(FStyleColors::Input, 2.0f))
			.SetCheckedHoveredImage(     FSlateRoundedBoxBrush(FStyleColors::Input, 2.0f))
			.SetCheckedPressedImage(     FSlateRoundedBoxBrush(FStyleColors::Input, 2.0f))

			.SetUncheckedImage(          FSlateRoundedBoxBrush(FStyleColors::Secondary, 2.0f))
			.SetUncheckedHoveredImage(   FSlateRoundedBoxBrush(FStyleColors::Hover, 2.0f))
			.SetUncheckedPressedImage(   FSlateRoundedBoxBrush(FStyleColors::Secondary, 2.0f))

			.SetForegroundColor(         FStyleColors::Foreground)
			.SetHoveredForegroundColor(  FStyleColors::ForegroundHover)
			.SetPressedForegroundColor(  FStyleColors::ForegroundHover)
			.SetCheckedForegroundColor(  FStyleColors::Primary)
			.SetCheckedHoveredForegroundColor(FStyleColors::PrimaryHover)
			.SetPadding(FMargin(2.f, 6.f))
		);

		Set("VerticalToolBar.MaxUniformToolbarSize", 48.f);
		Set("VerticalToolBar.MinUniformToolbarSize", 48.f);

		Set("VerticalToolBar", VerticalToolBarStyle);
	}
	
	// Ctrl+Tab menu
	{
		Set("ControlTabMenu.Background", new BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f)));

		Set("ControlTabMenu.HeadingStyle",
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 14))
			.SetColorAndOpacity(FLinearColor::White)
			);

		Set("ControlTabMenu.AssetTypeStyle",
			FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FLinearColor::White)
			);

		Set("ControlTabMenu.AssetPathStyle",
			FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FLinearColor::White)
			);

		Set("ControlTabMenu.AssetNameStyle",
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 14))
			.SetColorAndOpacity(FLinearColor::White)
			);
	}

	

	// ViewportLayoutToolbar
	{
		FToolBarStyle ViewportLayoutToolbar = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar");
		ViewportLayoutToolbar.SetIconSize(Icon32x32);
		ViewportLayoutToolbar.SetBackground(FSlateColorBrush(FStyleColors::Dropdown));

		Set("ViewportLayoutToolbar", ViewportLayoutToolbar);
	}

	
	// Scalability 
	{
		const float Tint = 0.65f;
		Set("Scalability.RowBackground", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f), FLinearColor(Tint, Tint, Tint) ) );
		Set("Scalability.TitleFont", DEFAULT_FONT( "Bold", 12 ) );
		Set("Scalability.GroupFont", DEFAULT_FONT( "Bold", 10 ) );
	}

#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

	// Gameplay Tags
	{
		Set("GameplayTagTreeView", FTableRowStyle()
			.SetEvenRowBackgroundBrush(FSlateNoResource())
			.SetEvenRowBackgroundHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetOddRowBackgroundBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetSelectorFocusedBrush(FSlateNoResource())
			.SetActiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor))
			.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, SelectionColor_Inactive))
			);
	}


	// Common styles for blueprint/code references
	{
		// Inherited from blueprint
		Set("Common.InheritedFromBlueprintTextColor", InheritedFromBlueprintTextColor);

		FTextBlockStyle InheritedFromBlueprintTextStyle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(InheritedFromBlueprintTextColor);

		Set("Common.InheritedFromBlueprintTextStyle", InheritedFromBlueprintTextStyle);

		// Go to blueprint hyperlink
		FButtonStyle EditBPHyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), InheritedFromBlueprintTextColor))
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), InheritedFromBlueprintTextColor));
		FHyperlinkStyle EditBPHyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(EditBPHyperlinkButton)
			.SetTextStyle(InheritedFromBlueprintTextStyle)
			.SetPadding(FMargin(0.0f));

		Set("Common.GotoBlueprintHyperlink", EditBPHyperlinkStyle);
	}

	// Timecode Provider
	{
		Set("TimecodeProvider.TabIcon", new IMAGE_BRUSH("Icons/icon_tab_TimecodeProvider_16x", Icon16x16));
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
}

void FStarshipEditorStyle::FStyle::SetupLevelGeneralStyles()
{
// Levels General
	{
		Set("Level.VisibleIcon16x", new CORE_IMAGE_BRUSH_SVG("Starship/Common/visible", Icon16x16));
		Set("Level.VisibleHighlightIcon16x", new CORE_IMAGE_BRUSH_SVG("Starship/Common/visible", Icon16x16));
		Set("Level.NotVisibleIcon16x", new CORE_IMAGE_BRUSH_SVG("Starship/Common/hidden", Icon16x16));
		Set("Level.NotVisibleHighlightIcon16x", new CORE_IMAGE_BRUSH_SVG("Starship/Common/hidden", Icon16x16));

		Set("Level.LightingScenarioIcon16x", new IMAGE_BRUSH_SVG("Starship/AssetIcons/PointLight_16", Icon16x16));
		Set("Level.LightingScenarioNotIcon16x", new IMAGE_BRUSH_SVG("Starship/Common/LightBulbOff", Icon16x16));
		Set("Level.LightingScenarioIconSolid16x", new IMAGE_BRUSH_SVG("Starship/Common/LightBulbSolid", Icon16x16));
		Set("Level.LightingScenarioNotIconSolid16x", new IMAGE_BRUSH_SVG("Starship/Common/LightBulbOffSolid", Icon16x16));
		Set("Level.LockedIcon16x", new IMAGE_BRUSH("Icons/icon_locked_16px", Icon16x16));
		Set("Level.LockedHighlightIcon16x", new IMAGE_BRUSH("Icons/icon_locked_highlight_16px", Icon16x16));
		Set("Level.UnlockedIcon16x", new IMAGE_BRUSH("Icons/icon_levels_unlocked_16px", Icon16x16));
		Set("Level.UnlockedHighlightIcon16x", new IMAGE_BRUSH("Icons/icon_levels_unlocked_hi_16px", Icon16x16));
		Set("Level.ReadOnlyLockedIcon16x", new IMAGE_BRUSH("Icons/icon_levels_LockedReadOnly_16px", Icon16x16));
		Set("Level.ReadOnlyLockedHighlightIcon16x", new IMAGE_BRUSH("Icons/icon_levels_LockedReadOnly_hi_16px", Icon16x16));
		Set("Level.SaveIcon16x", new IMAGE_BRUSH("Icons/icon_levels_Save_16px", Icon16x16));
		Set("Level.SaveHighlightIcon16x", new IMAGE_BRUSH("Icons/icon_levels_Save_hi_16px", Icon16x16));
		Set("Level.SaveModifiedIcon16x", new IMAGE_BRUSH("Icons/icon_levels_SaveModified_16px", Icon16x16));
		Set("Level.SaveModifiedHighlightIcon16x", new IMAGE_BRUSH("Icons/icon_levels_SaveModified_hi_16px", Icon16x16));
		Set("Level.SaveDisabledIcon16x", new IMAGE_BRUSH("Icons/icon_levels_SaveDisabled_16px", Icon16x16));
		Set("Level.SaveDisabledHighlightIcon16x", new IMAGE_BRUSH("Icons/icon_levels_SaveDisabled_hi_16px", Icon16x16));
		Set("Level.ScriptIcon16x", new IMAGE_BRUSH("Icons/icon_levels_Blueprint_16px", Icon16x16));
		Set("Level.ScriptHighlightIcon16x", new IMAGE_BRUSH("Icons/icon_levels_Blueprint_hi_16px", Icon16x16));
		Set("Level.EmptyIcon16x", new IMAGE_BRUSH("Icons/Empty_16x", Icon16x16));
		Set("Level.ColorIcon", new FSlateRoundedBoxBrush(FStyleColors::White, 4.0f, Icon16x16));
	}

	// Spline component controls
	{
		Set("SplineComponentDetails.SelectFirst", FButtonStyle(Button)
			.SetNormal(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectFirst", Icon20x20))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectFirst", Icon20x20, SelectionColor))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectFirst", Icon20x20, SelectionColor_Pressed))
			.SetDisabled(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectFirst", Icon20x20, SelectionColor_Inactive))
		);

		Set("SplineComponentDetails.AddPrev", FButtonStyle(Button)
			.SetNormal(IMAGE_BRUSH_SVG("Starship/Splines/Spline_AddPrevious", Icon20x20))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/Splines/Spline_AddPrevious", Icon20x20, SelectionColor))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/Splines/Spline_AddPrevious", Icon20x20, SelectionColor_Pressed))
			.SetDisabled(IMAGE_BRUSH_SVG("Starship/Splines/Spline_AddPrevious", Icon20x20, SelectionColor_Inactive))
		);

		Set("SplineComponentDetails.SelectPrev", FButtonStyle(Button)
			.SetNormal(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectPrevious", Icon20x20))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectPrevious", Icon20x20, SelectionColor))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectPrevious", Icon20x20, SelectionColor_Pressed))
			.SetDisabled(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectPrevious", Icon20x20, SelectionColor_Inactive))
		);

		Set("SplineComponentDetails.SelectAll", FButtonStyle(Button)
			.SetNormal(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectAll", Icon20x20))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectAll", Icon20x20, SelectionColor))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectAll", Icon20x20, SelectionColor_Pressed))
			.SetDisabled(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectAll", Icon20x20, SelectionColor_Inactive))
		);

		Set("SplineComponentDetails.SelectNext", FButtonStyle(Button)
			.SetNormal(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectNext", Icon20x20))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectNext", Icon20x20, SelectionColor))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectNext", Icon20x20, SelectionColor_Pressed))
			.SetDisabled(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectNext", Icon20x20, SelectionColor_Inactive))
		);

		Set("SplineComponentDetails.AddNext", FButtonStyle(Button)
			.SetNormal(IMAGE_BRUSH_SVG("Starship/Splines/Spline_AddNext", Icon20x20))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/Splines/Spline_AddNext", Icon20x20, SelectionColor))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/Splines/Spline_AddNext", Icon20x20, SelectionColor_Pressed))
			.SetDisabled(IMAGE_BRUSH_SVG("Starship/Splines/Spline_AddNext", Icon20x20, SelectionColor_Inactive))
		);

		Set("SplineComponentDetails.SelectLast", FButtonStyle(Button)
			.SetNormal(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectLast", Icon20x20))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectLast", Icon20x20, SelectionColor))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectLast", Icon20x20, SelectionColor_Pressed))
			.SetDisabled(IMAGE_BRUSH_SVG("Starship/Splines/Spline_SelectLast", Icon20x20, SelectionColor_Inactive))
		);

		Set("SplineComponentDetails.ConvertToSegments", FButtonStyle(Button)
			.SetNormal(IMAGE_BRUSH_SVG("Starship/Splines/Spline_Segment", Icon20x20, FSlateColor::UseForeground()))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/Splines/Spline_Segment", Icon20x20, SelectionColor))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/Splines/Spline_Segment", Icon20x20, SelectionColor_Pressed))
			.SetDisabled(IMAGE_BRUSH_SVG("Starship/Splines/Spline_Segment", Icon20x20, SelectionColor_Inactive))
		);

		Set("SplineComponentDetails.ConvertToPoints", FButtonStyle(Button)
			.SetNormal(IMAGE_BRUSH_SVG("Starship/Splines/Spline_ControlPoint", Icon20x20, FSlateColor::UseForeground()))
			.SetHovered(IMAGE_BRUSH_SVG("Starship/Splines/Spline_ControlPoint", Icon20x20, SelectionColor))
			.SetPressed(IMAGE_BRUSH_SVG("Starship/Splines/Spline_ControlPoint", Icon20x20, SelectionColor_Pressed))
			.SetDisabled(IMAGE_BRUSH_SVG("Starship/Splines/Spline_ControlPoint", Icon20x20, SelectionColor_Inactive))
		);
	}
}

void FStarshipEditorStyle::FStyle::SetupWorldBrowserStyles()
{
	// World Browser
	{
		Set("WorldBrowser.AddLayer", new IMAGE_BRUSH("Icons/icon_levels_addlayer_16x", Icon16x16));
		Set("WorldBrowser.SimulationViewPosition", new IMAGE_BRUSH("Icons/icon_levels_simulationviewpos_16x", Icon16x16));
		Set("WorldBrowser.MouseLocation", new IMAGE_BRUSH("Icons/icon_levels_mouselocation_16x", Icon16x16));
		Set("WorldBrowser.MarqueeRectSize", new IMAGE_BRUSH("Icons/icon_levels_marqueerectsize_16x", Icon16x16));
		Set("WorldBrowser.WorldSize", new IMAGE_BRUSH("Icons/icon_levels_worldsize_16x", Icon16x16));
		Set("WorldBrowser.WorldOrigin", new IMAGE_BRUSH("Icons/icon_levels_worldorigin_16x", Icon16x16));
		Set("WorldBrowser.DirectionXPositive", new IMAGE_BRUSH("Icons/icon_PanRight", Icon16x16));
		Set("WorldBrowser.DirectionXNegative", new IMAGE_BRUSH("Icons/icon_PanLeft", Icon16x16));
		Set("WorldBrowser.DirectionYPositive", new IMAGE_BRUSH("Icons/icon_PanUp", Icon16x16));
		Set("WorldBrowser.DirectionYNegative", new IMAGE_BRUSH("Icons/icon_PanDown", Icon16x16));
		Set("WorldBrowser.LevelStreamingAlwaysLoaded", new FSlateNoResource());
		Set("WorldBrowser.LevelStreamingBlueprint", new IMAGE_BRUSH("Icons/icon_levels_blueprinttype_7x16", Icon7x16));

		Set("WorldBrowser.LevelsMenuBrush", new IMAGE_BRUSH_SVG("Starship/WorldBrowser/LevelStack_20", Icon20x20));
		Set("WorldBrowser.DetailsButtonBrush", new IMAGE_BRUSH_SVG("Starship/Common/Details", Icon20x20) );
		Set("WorldBrowser.HierarchyButtonBrush", new IMAGE_BRUSH_SVG("Starship/WorldBrowser/LevelStack_20", Icon20x20));

		Set("WorldBrowser.CompositionButtonBrush", new IMAGE_BRUSH_SVG("Starship/WorldBrowser/WorldComp_20", Icon20x20));
		Set("WorldBrowser.NewFolderIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-plus", Icon16x16));

		Set("WorldBrowser.StatusBarText", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("BoldCondensed", 12))
			.SetColorAndOpacity(FLinearColor(0.9f, 0.9f, 0.9f, 0.5f))
			.SetShadowOffset(FVector2f::ZeroVector)
		);

		Set("WorldBrowser.LabelFont", DEFAULT_FONT("Regular", 9));
		Set("WorldBrowser.LabelFontBold", DEFAULT_FONT("Bold", 10));
	}
}

void FStarshipEditorStyle::FStyle::SetupWorldPartitionStyles()
{
	// World Partition
	Set("WorldPartition.SimulationViewPosition", new IMAGE_BRUSH("Icons/icon_levels_simulationviewpos_16x", Icon16x16));

	Set("WorldPartition.FollowPlayerInPIE", new IMAGE_BRUSH_SVG("WorldPartition//Developer_20", Icon16x16));
	Set("WorldPartition.BugItGoLoadRegion", new IMAGE_BRUSH_SVG("WorldPartition//Bug_20", Icon16x16)); 

	Set("WorldPartition.ShowActors", new IMAGE_BRUSH_SVG("WorldPartition/Actor_20", Icon16x16));
	Set("WorldPartition.ShowHLODActors", new IMAGE_BRUSH_SVG("Starship/Common/HierarchicalLOD", Icon16x16));
	Set("WorldPartition.ShowGrid", new IMAGE_BRUSH_SVG("Starship/EditorViewport/grid", Icon16x16));
	Set("WorldPartition.ShowMiniMap", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Texture2D_16", Icon16x16)); 
	Set("WorldPartition.ShowCoords", new IMAGE_BRUSH_SVG("WorldPartition/Coordinate_16", Icon16x16));
	Set("WorldPartition.ShowLoadingRegions", new IMAGE_BRUSH_SVG("Starship/Common/Volumes", Icon16x16));
	Set("WorldPartition.ShowMouseCoords", new IMAGE_BRUSH_SVG("WorldPartition/MouseCoordinates_16", Icon16x16));
	Set("WorldPartition.FocusSelection", new IMAGE_BRUSH_SVG("WorldPartition/FramingSelection_20", Icon16x16));
	Set("WorldPartition.FocusLoadedRegions", new IMAGE_BRUSH_SVG("WorldPartition/FrameRegions_20", Icon16x16));
	Set("WorldPartition.FocusWorld", new IMAGE_BRUSH_SVG("WorldPartition/FrameWorld_20", Icon16x16));

	// Level Instance
	Set("LevelInstance.ColumnOverrideHereEditable", new IMAGE_BRUSH_SVG("Starship/LevelInstance/ColumnOverrideHere", Icon16x16, FStyleColors::AccentBlue));
	Set("LevelInstance.ColumnOverrideHere", new IMAGE_BRUSH_SVG("Starship/LevelInstance/ColumnOverrideHere", Icon16x16, FStyleColors::AccentGray));

	Set("LevelInstance.ColumnOverrideContainerHereEditable", new IMAGE_BRUSH_SVG("Starship/LevelInstance/ColumnOverrideContainerHere", Icon16x16, FStyleColors::AccentBlue));
	Set("LevelInstance.ColumnOverrideContainerHere", new IMAGE_BRUSH_SVG("Starship/LevelInstance/ColumnOverrideContainerHere", Icon16x16, FStyleColors::AccentGray));

	Set("LevelInstance.ColumnOverrideContainerInsideEditable", new IMAGE_BRUSH_SVG("Starship/LevelInstance/ColumnOverrideContainerInside", Icon16x16, FStyleColors::AccentBlue));
	Set("LevelInstance.ColumnOverrideContainerInside", new IMAGE_BRUSH_SVG("Starship/LevelInstance/ColumnOverrideContainerInside", Icon16x16, FStyleColors::AccentGray));

	Set("LevelInstance.ColumnOverrideContainerEditable", new IMAGE_BRUSH_SVG("Starship/LevelInstance/ColumnOverrideContainer", Icon16x16, FStyleColors::AccentBlue));
	Set("LevelInstance.ColumnOverrideContainer", new IMAGE_BRUSH_SVG("Starship/LevelInstance/ColumnOverrideContainer", Icon16x16, FStyleColors::AccentGray));
}

void FStarshipEditorStyle::FStyle::SetupSequencerStyles()
{
	// Sequencer
	if (IncludeEditorSpecificStyles())
	{
		FToolBarStyle SequencerToolbar = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("AssetEditorToolbar");

		SequencerToolbar.SetButtonPadding(       FMargin(2.0f, 0.0f));
		SequencerToolbar.SetCheckBoxPadding(     FMargin(0.0f, 0.0f));
		SequencerToolbar.SetComboButtonPadding(  FMargin(0.0f, 0.0f));
		SequencerToolbar.SetIndentedBlockPadding(FMargin(0.0f, 0.0f));
		SequencerToolbar.SetBlockPadding(        FMargin(0.0f, 0.0f));
		SequencerToolbar.SetSeparatorPadding(    FMargin(2.0f, 0.0f));

		Set("SequencerToolBar", SequencerToolbar);

		const FTableRowStyle AlternatingTableRowStyle = GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow");

		// Top parent hover is 2x brighter than header color
		FLinearColor BrighterHeader = FStyleColors::Header.GetSpecifiedColor().LinearRGBToHSV();
		BrighterHeader.B = FMath::Min(1.f, BrighterHeader.B * 2.0f);
		BrighterHeader = BrighterHeader.HSVToLinearRGB();

		Set("Sequencer.Outliner.Row", FTableRowStyle(AlternatingTableRowStyle)
			.SetUseParentRowBrush(true)
			.SetParentRowBackgroundBrush(FSlateColorBrush(FStyleColors::Header))
			.SetParentRowBackgroundHoveredBrush(FSlateColorBrush(BrighterHeader)));

		Set("Sequencer.Outliner.Separator", new FSlateColorBrush(FStyleColors::Input));
		Set("Sequencer.Outliner.Plus", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/Plus", Icon14x14));
		Set("Sequencer.Outliner.AddKey", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/AddKey", Icon14x14));
		Set("Sequencer.Outliner.NextKey", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/NextKey", Icon14x14));
		Set("Sequencer.Outliner.PreviousKey", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/PreviousKey", Icon14x14));
		Set("Sequencer.Outliner.CameraLock", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/SequencerCamera", Icon14x14));

		Set("Sequencer.Outliner.ColumnButton", FButtonStyle()
				.SetNormal(FSlateNoResource())
				.SetHovered(FSlateNoResource())
				.SetPressed(FSlateNoResource())
				.SetNormalPadding(FMargin(0,0,0,1))
				.SetPressedPadding(FMargin(0,1,0,0)) );

		Set("Sequencer.Outliner.ToggleButton", FCheckBoxStyle( GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
			.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Header, 4.0f, FStyleColors::Input, 1.0f))
			.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f))
			.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::Input, 1.0f))
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f, FStyleColors::Input, 1.0f))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f, FStyleColors::Input, 1.0f))
			.SetPadding(FMargin(6.f, 1.f))
		);

		Set("Sequencer.IconKeySmartAuto", new IMAGE_BRUSH("Sequencer/IconKeySmartAuto", Icon12x12));
		Set("Sequencer.IconKeyAuto", new IMAGE_BRUSH("Sequencer/IconKeyAuto", Icon12x12));
		Set("Sequencer.IconKeyBreak", new IMAGE_BRUSH("Sequencer/IconKeyBreak", Icon12x12));
		Set("Sequencer.IconKeyConstant", new IMAGE_BRUSH("Sequencer/IconKeyConstant", Icon12x12));
		Set("Sequencer.IconKeyLinear", new IMAGE_BRUSH("Sequencer/IconKeyLinear", Icon12x12));
		Set("Sequencer.IconKeyUser", new IMAGE_BRUSH("Sequencer/IconKeyUser", Icon12x12));

		Set("Sequencer.KeyCircle", new IMAGE_BRUSH("Sequencer/KeyCircle", Icon12x12));
		Set("Sequencer.KeyDiamond", new IMAGE_BRUSH("Sequencer/KeyDiamond", Icon12x12));
		Set("Sequencer.KeyDiamondBorder", new IMAGE_BRUSH("Sequencer/KeyDiamondBorder", Icon12x12));
		Set("Sequencer.KeySquare", new IMAGE_BRUSH("Sequencer/KeySquare", Icon12x12));
		Set("Sequencer.KeyTriangle", new IMAGE_BRUSH("Sequencer/KeyTriangle", Icon12x12));
		Set("Sequencer.KeyTriangle20", new CORE_IMAGE_BRUSH_SVG("Starship/Common/curve-editor-append-key-20", Icon20x20));
		Set("Sequencer.KeyLeft", new IMAGE_BRUSH("Sequencer/KeyLeft", Icon12x12));
		Set("Sequencer.KeyRight", new IMAGE_BRUSH("Sequencer/KeyRight", Icon12x12));
		Set("Sequencer.PartialKey", new IMAGE_BRUSH("Sequencer/PartialKey", FVector2f(11.f, 11.f)));
		Set("Sequencer.Star", new IMAGE_BRUSH("Sequencer/Star", Icon12x12));
		Set("Sequencer.Empty", new IMAGE_BRUSH("Sequencer/Empty", Icon12x12));
		Set("Sequencer.TangentHandle", new IMAGE_BRUSH("Sequencer/TangentHandle", FVector2f(7.f, 7.f)));
		Set("Sequencer.GenericDivider", new IMAGE_BRUSH("Sequencer/GenericDivider", FVector2f(2.f, 2.f), FLinearColor::White, ESlateBrushTileType::Vertical));

		Set("Sequencer.KeyBar.Dotted", new BORDER_BRUSH(TEXT("Sequencer/Keys/KeyBar_Dotted"), FMargin(0.f, 4.f, 0.f, 0.f)));
		Set("Sequencer.KeyBar.Dashed", new BORDER_BRUSH(TEXT("Sequencer/Keys/KeyBar_Dashed"), FMargin(0.f, 4.f, 0.f, 0.f)));
		Set("Sequencer.KeyBar.Solid", new BORDER_BRUSH(TEXT("Sequencer/Keys/KeyBar_Solid"), FMargin(0.f, 4.f, 0.f, 0.f)));

		Set("Sequencer.Timeline.ScrubHandle", new IMAGE_BRUSH_SVG("Starship/Sequencer/ScrubHandle", FVector2f(14.f, 32.f)));
		Set("Sequencer.Timeline.ScrubFill", new BOX_BRUSH("Sequencer/ScrubFill", FMargin(2.f / 4.f, 0.f)));
		Set("Sequencer.Timeline.FrameBlockScrubHandle", new BOX_BRUSH("Sequencer/ScrubHandleDown", FMargin(6.f / 13.f, 5 / 12.f, 6 / 13.f, 8 / 12.f)));
		Set("Sequencer.Timeline.ScrubHandleWhole", new BOX_BRUSH("Sequencer/ScrubHandleWhole", FMargin(6.f / 13.f, 10 / 24.f, 6 / 13.f, 10 / 24.f)));
		Set("Sequencer.Timeline.RangeHandleLeft", new BOX_BRUSH("Sequencer/GenericGripLeft", FMargin(5.f / 16.f)));
		Set("Sequencer.Timeline.RangeHandleRight", new BOX_BRUSH("Sequencer/GenericGripRight", FMargin(5.f / 16.f)));
		Set("Sequencer.Timeline.RangeHandle", new BOX_BRUSH("Sequencer/GenericSectionBackground", FMargin(5.f / 16.f)));
		Set("Sequencer.Timeline.NotifyAlignmentMarker", new IMAGE_BRUSH("Sequencer/NotifyAlignmentMarker", FVector2f(10.f, 19.f)));
		Set("Sequencer.Timeline.PlayRange_Top_L", new BOX_BRUSH("Sequencer/PlayRange_Top_L", FMargin(1.f, 0.5f, 0.f, 0.5f)));
		Set("Sequencer.Timeline.PlayRange_Top_R", new BOX_BRUSH("Sequencer/PlayRange_Top_R", FMargin(0.f, 0.5f, 1.f, 0.5f)));
		Set("Sequencer.Timeline.PlayRange_L", new BOX_BRUSH("Sequencer/PlayRange_L", FMargin(1.f, 0.5f, 0.f, 0.5f)));
		Set("Sequencer.Timeline.PlayRange_R", new BOX_BRUSH("Sequencer/PlayRange_R", FMargin(0.f, 0.5f, 1.f, 0.5f)));
		Set("Sequencer.Timeline.PlayRange_Bottom_L", new BOX_BRUSH("Sequencer/PlayRange_Bottom_L", FMargin(1.f, 0.5f, 0.f, 0.5f)));
		Set("Sequencer.Timeline.PlayRange_Bottom_R", new BOX_BRUSH("Sequencer/PlayRange_Bottom_R", FMargin(0.f, 0.5f, 1.f, 0.5f)));

		Set("Sequencer.Timeline.SubSequenceRangeHashL", new BORDER_BRUSH("Sequencer/SubSequenceRangeHashL", FMargin(1.f, 0.f, 0.f, 0.f)));
		Set("Sequencer.Timeline.SubSequenceRangeHashR", new BORDER_BRUSH("Sequencer/SubSequenceRangeHashR", FMargin(1.f, 0.f, 0.f, 0.f)));
		Set("Sequencer.Timeline.EaseInOut", new IMAGE_BRUSH("Sequencer/EaseInOut", Icon128x128));
		Set("Sequencer.InterpLine", new BOX_BRUSH("Sequencer/InterpLine", FMargin(5.f / 7.f, 0.f, 0.f, 0.f)));

		Set("Sequencer.Transport.JumpToPreviousKey", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsJumpToPreviousKey", Icon20x20));
		Set("Sequencer.Transport.JumpToNextKey", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsJumpToNextKey", Icon20x20));
		Set("Sequencer.Transport.SetPlayStart", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsSetPlaybackStart", Icon20x20));
		Set("Sequencer.Transport.SetPlayEnd", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsSetPlaybackEnd", Icon20x20));
		Set("Sequencer.Transport.Looping", new IMAGE_BRUSH_SVG("Sequencer/PlaybackControls/PlayControlsLooping", Icon20x20));

		Set("Sequencer.Transport.CloseButton", FButtonStyle()
			.SetNormal(IMAGE_BRUSH("/Docking/CloseApp_Normal", Icon16x16))
			.SetPressed(IMAGE_BRUSH("/Docking/CloseApp_Pressed", Icon16x16))
			.SetHovered(IMAGE_BRUSH("/Docking/CloseApp_Hovered", Icon16x16)));

		Set("Sequencer.NotificationImage_AddedPlayMovieSceneEvent", new IMAGE_BRUSH("Old/Checkbox_checked", Icon16x16));

		Set("Sequencer.Timeline.ScrubHandleDown", new BOX_BRUSH("Sequencer/ScrubHandleDown", FMargin(6.f / 13.f, 5 / 12.f, 6 / 13.f, 8 / 12.f)));
		Set("Sequencer.Timeline.ScrubHandleUp", new BOX_BRUSH("Sequencer/ScrubHandleUp", FMargin(6.f / 13.f, 8 / 12.f, 6 / 13.f, 5 / 12.f)));
		Set("Sequencer.Timeline.VanillaScrubHandleDown", new BOX_BRUSH("Sequencer/ScrubHandleDown_Clamped", FMargin(6.f / 13.f, 3.f / 12.f, 6.f / 13.f, 7.f / 12.f)));
		Set("Sequencer.Timeline.VanillaScrubHandleUp", new BOX_BRUSH("Sequencer/ScrubHandleUp_Clamped", FMargin(6.f / 13.f, 8 / 12.f, 6 / 13.f, 5 / 12.f)));

		Set("Sequencer.RestoreAnimatedState", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_Sequencer_RestoreAnimatedState_24x", Icon48x48));
		Set("Sequencer.GenericGripLeft", new BOX_BRUSH("Sequencer/GenericGripLeft", FMargin(5.f / 16.f)));
		Set("Sequencer.GenericGripRight", new BOX_BRUSH("Sequencer/GenericGripRight", FMargin(5.f / 16.f)));
		Set("Sequencer.SectionArea.Background", new FSlateColorBrush(FStyleColors::White));

		Set("Sequencer.Section.Background", new BORDER_BRUSH(TEXT("Sequencer/SectionBackground"), FMargin(4.f / 16.f)));
		Set("Sequencer.Section.BackgroundTint", new BOX_BRUSH(TEXT("Sequencer/SectionBackgroundTint"), FMargin(4 / 16.f)));
		Set("Sequencer.Section.CollapsedSelectedSectionOverlay", new IMAGE_BRUSH(TEXT("Sequencer/Section/CollapsedSelectedSectionOverlay"), Icon16x16, FLinearColor::White, ESlateBrushTileType::Both));
		Set("Sequencer.Section.SequencerDeactivatedOverlay", new IMAGE_BRUSH(TEXT("Sequencer/Section/SequencerDeactivatedOverlay"), Icon16x16, FLinearColor::White, ESlateBrushTileType::Both));
		Set("Sequencer.Section.ErroredSectionOverlay", new BORDER_BRUSH(TEXT("Sequencer/Section/CollapsedSelectedSectionOverlay"), FMargin(4.f / 16.f)));
		Set("Sequencer.Section.SectionHeaderSelectedSectionOverlay", new IMAGE_BRUSH(TEXT("Sequencer/Section/SectionHeaderSelectedSectionOverlay"), Icon16x16, FLinearColor::White, ESlateBrushTileType::Both));
		Set("Sequencer.Section.SelectedTrackTint", new BOX_BRUSH(TEXT("Sequencer/SelectedTrackTint"), FMargin(0.f, 0.5f)));
		Set("Sequencer.Section.SelectionBorder", new BORDER_BRUSH(TEXT("Sequencer/SectionHighlight"), FMargin(7.f / 16.f)));
		Set("Sequencer.Section.LockedBorder", new BORDER_BRUSH(TEXT("Sequencer/SectionLocked"), FMargin(7.f / 16.f)));
		Set("Sequencer.Section.FilmBorder", new IMAGE_BRUSH(TEXT("Sequencer/SectionFilmBorder"), FVector2f(10.f, 7.f), FLinearColor::White, ESlateBrushTileType::Horizontal));
		Set("Sequencer.Section.GripLeft", new FSlateRoundedBoxBrush(FStyleColors::White, FVector4(4.f, 0.f, 0.f, 4.f)));
		Set("Sequencer.Section.GripRight", new FSlateRoundedBoxBrush(FStyleColors::White, FVector4(0.f, 4.f, 4.f, 0.f)));
		Set("Sequencer.Section.EasingHandle", new FSlateColorBrush(FStyleColors::White));

		Set("Sequencer.Section.Background_Collapsed", new FSlateRoundedBoxBrush(FLinearColor::White, FVector4(4.f, 4.f, 4.f, 4.f)));
		Set("Sequencer.Section.Background_Header", new FSlateRoundedBoxBrush(FLinearColor::White, FVector4(4.f, 4.f, 0.f, 0.f)));
		Set("Sequencer.Section.Background_Contents", new FSlateRoundedBoxBrush(FLinearColor::White, FVector4(0.f, 0.f, 4.f, 4.f)));

		Set("Sequencer.Section.PreRoll", new BORDER_BRUSH(TEXT("Sequencer/PreRoll"), FMargin(0.f, .5f, 0.f, .5f)));

		Set("Sequencer.Section.PinCusion", new IMAGE_BRUSH(TEXT("Sequencer/PinCusion"), Icon16x16, FLinearColor::White, ESlateBrushTileType::Both));
		Set("Sequencer.Section.OverlapBorder", new BORDER_BRUSH(TEXT("Sequencer/OverlapBorder"), FMargin(1.f / 4.f, 0.f)));
		Set("Sequencer.Section.StripeOverlay", new BOX_BRUSH("Sequencer/SectionStripeOverlay", FMargin(0.f, .5f)));
		Set("Sequencer.Section.BackgroundText", DEFAULT_FONT("Bold", 24));
		Set("Sequencer.Section.EmptySpace", new BOX_BRUSH(TEXT("Sequencer/EmptySpace"), FMargin(0.f, 7.f / 14.f)));

		Set("Sequencer.MarkedFrame.LabelRight", new FSlateRoundedBoxBrush(FLinearColor::White, FVector4(0.f, 0.f, 4.f, 0.f)));
		Set("Sequencer.MarkedFrame.LabelLeft", new FSlateRoundedBoxBrush(FLinearColor::White, FVector4(0.f, 0.f, 0.f, 4.f)));

		Set("Sequencer.LayerBar.Background",  new BOX_BRUSH(TEXT("Sequencer/LayerBarBackground"), FMargin(4.f / 16.f)));
		Set("Sequencer.LayerBar.HandleLeft",  new FSlateColorBrush(FStyleColors::White)); // Maybe use a rounded box if we decide to round-out the layer bars
		Set("Sequencer.LayerBar.HandleRight", new FSlateColorBrush(FStyleColors::White)); // Maybe use a rounded box in we decide to round-out the layer bars

		Set("Sequencer.ExposedNamePill_BG", new BOX_BRUSH(TEXT("Sequencer/ExposedNamePill_BG"), FMargin(14.f / 30.f), FLinearColor(1.f, 1.f, 1.f, .8f)));
		Set("Sequencer.ExposedNamePill", FButtonStyle()
			.SetNormal(  BOX_BRUSH(TEXT("Sequencer/ExposedNamePill_BG"), FMargin(14.f / 30.f), FLinearColor(1.f, 1.f, 1.f, .8f)) )
			.SetHovered( BOX_BRUSH(TEXT("Sequencer/ExposedNamePill_BG_Hovered"), FMargin(14.f / 30.f), FLinearColor::White) )
			.SetPressed( BOX_BRUSH(TEXT("Sequencer/ExposedNamePill_BG_Pressed"), FMargin(14.f / 30.f), FLinearColor::White) )
			.SetNormalPadding( FMargin(0,0,0,0) )
			.SetPressedPadding( FMargin(0,0,0,0) )
		);

		Set("Sequencer.AnimationOutliner.ColorStrip", FButtonStyle()
			.SetNormal(FSlateNoResource())
			.SetHovered(FSlateNoResource())
			.SetPressed(FSlateNoResource())
			.SetNormalPadding(FMargin(0, 0, 0, 0))
			.SetPressedPadding(FMargin(0, 0, 0, 0))
		);

		Set("Sequencer.AnimationOutliner.TopLevelBorder_Expanded", new BOX_BRUSH("Sequencer/TopLevelNodeBorder_Expanded", FMargin(4.0f / 16.0f)));
		Set("Sequencer.AnimationOutliner.TopLevelBorder_Collapsed", new BOX_BRUSH("Sequencer/TopLevelNodeBorder_Collapsed", FMargin(4.0f / 16.0f)));
		Set("Sequencer.AnimationOutliner.DefaultBorder", new FSlateColorBrush(FLinearColor::White));
		Set("Sequencer.AnimationOutliner.TransparentBorder", new FSlateColorBrush(FLinearColor::Transparent));
		Set("Sequencer.AnimationOutliner.BoldFont", DEFAULT_FONT("Bold", 11));
		Set("Sequencer.AnimationOutliner.RegularFont", DEFAULT_FONT("Regular", 9));
		Set("Sequencer.AnimationOutliner.ItalicFont", DEFAULT_FONT("Italic", 10));

		Set("Sequencer.Outliner.Indicators.TimeWarp", new IMAGE_BRUSH_SVG("Starship/Sequencer/TimeWarp_12", Icon12x12));
		
		FCheckBoxStyle BaseCheckBoxStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
		Set("Sequencer.LinkedFilterToggleButton", BaseCheckBoxStyle
			.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f))
			.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f))
			.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f))
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f))
			.SetPadding(FMargin(2, 2))
		);
		Set("Sequencer.LinkedFiltering.LinkedIcon", new IMAGE_BRUSH_SVG("Starship/Sequencer/FilteringLinked_16", Icon16x16));
		Set("Sequencer.LinkedFiltering.InstancedIcon", new IMAGE_BRUSH_SVG("Starship/Sequencer/FilteringInstanced_16", Icon16x16));

		Set("Sequencer.ShotFilter", new IMAGE_BRUSH("Sequencer/FilteredArea", FVector2f(74.f, 74.f), FLinearColor::White, ESlateBrushTileType::Both));
		Set("Sequencer.KeyMark", new IMAGE_BRUSH("Sequencer/KeyMark", FVector2f(3.f, 21.f), FLinearColor::White, ESlateBrushTileType::NoTile));
		Set("Sequencer.ToggleAutoKeyEnabled", new IMAGE_BRUSH_SVG("Starship/Sequencer/AutoKey", Icon20x20));
		Set("Sequencer.SetAutoKey", new IMAGE_BRUSH_SVG("Starship/Sequencer/AutoKey", Icon20x20));
		Set("Sequencer.SetAutoTrack", new IMAGE_BRUSH_SVG("Starship/Sequencer/AutoTrack", Icon20x20));
		Set("Sequencer.SetAutoChangeAll", new IMAGE_BRUSH_SVG("Starship/Sequencer/AutoChangeAll", Icon20x20));
		Set("Sequencer.SetAutoChangeNone", new IMAGE_BRUSH_SVG("Starship/Sequencer/AutoChangeNone", Icon20x20));
		Set("Sequencer.AllowAllEdits", new IMAGE_BRUSH_SVG("Starship/Sequencer/AllowAllEdits", Icon20x20));
		Set("Sequencer.AllowSequencerEditsOnly", new IMAGE_BRUSH_SVG("Starship/Sequencer/AllowSequencerEditsOnly", Icon20x20));
		Set("Sequencer.AllowLevelEditsOnly", new IMAGE_BRUSH_SVG("Starship/Sequencer/AllowLevelEditsOnly", Icon20x20));
		Set("Sequencer.SetKeyAll", new IMAGE_BRUSH_SVG("Starship/Sequencer/KeyAll", Icon20x20));
		Set("Sequencer.SetKeyGroup", new IMAGE_BRUSH_SVG("Starship/Sequencer/KeyGroup", Icon20x20));
		Set("Sequencer.SetKeyChanged", new IMAGE_BRUSH_SVG("Starship/Sequencer/KeyChanged", Icon20x20));
		Set("Sequencer.ToggleIsSnapEnabled", new IMAGE_BRUSH_SVG("Starship/Sequencer/Snap", Icon20x20));
		Set("Sequencer.ToggleForceWholeFrames", new IMAGE_BRUSH_SVG("Starship/Sequencer/ForceWholeFrames", Icon20x20));
		Set("Sequencer.ToggleLimitViewportSelection",
			new IMAGE_BRUSH_SVG("Starship/Sequencer/SelectOnlyInSequence_16", Icon20x20));

		Set("Sequencer.OpenTaggedBindingManager", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_Sequencer_OpenTaggedBindingManager_16x", Icon48x48));
		Set("Sequencer.OpenNodeGroupsManager", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_Sequencer_OpenGroupManager_16x", Icon48x48));
		Set("Sequencer.CreateCamera", new IMAGE_BRUSH_SVG("Starship/Sequencer/CreateCamera", Icon20x20));
		Set("Sequencer.LockCamera", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_Sequencer_Look_Thru_24x", Icon16x16));
		Set("Sequencer.UnlockCamera", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_Sequencer_Look_Thru_24x", Icon16x16, FLinearColor(1.f, 1.f, 1.f, 0.5f)));
		Set("Sequencer.Thumbnail.SectionHandle", new IMAGE_BRUSH("Old/White", Icon16x16, FLinearColor::Black));
		Set("Sequencer.TrackHoverHighlight_Top", new IMAGE_BRUSH(TEXT("Sequencer/TrackHoverHighlight_Top"), FVector2f(4.f, 4.f)));
		Set("Sequencer.TrackHoverHighlight_Bottom", new IMAGE_BRUSH(TEXT("Sequencer/TrackHoverHighlight_Bottom"), FVector2f(4.f, 4.f)));
		Set("Sequencer.SpawnableIconOverlay", new IMAGE_BRUSH(TEXT("Sequencer/SpawnableIconOverlay"), FVector2f(13.f, 13.f)));
		Set("Sequencer.ReplaceableIconOverlay", new IMAGE_BRUSH(TEXT("Sequencer/ReplaceableIconOverlay"), FVector2f(13.f, 13.f)));
		Set("Sequencer.MultipleIconOverlay", new IMAGE_BRUSH(TEXT("Sequencer/MultipleIconOverlay"), FVector2f(13.f, 13.f)));
		Set("Sequencer.ProxyIconOverlay", new IMAGE_BRUSH_SVG(TEXT("Sequencer/ProxyIconOverlay_13"), FVector2f(13.f, 13.f)));
		Set("Sequencer.DynamicBindingIconOverlay", new IMAGE_BRUSH(TEXT("Sequencer/DynamicBindingIconOverlay"), Icon16x16));
		Set("Sequencer.SpawnableDynamicBindingIconOverlay", new IMAGE_BRUSH(TEXT("Sequencer/SpawnableDynamicBindingIconOverlay"), Icon16x16));
		Set("Sequencer.LockSequence", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_Sequencer_Locked_16x", Icon16x16));
		Set("Sequencer.UnlockSequence", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_Sequencer_Unlocked_16x", Icon16x16));

		Set("Sequencer.Bake", new IMAGE_BRUSH_SVG(TEXT("Sequencer/Bake_14"), Icon14x14));
		Set("Sequencer.Weight", new IMAGE_BRUSH_SVG(TEXT("Sequencer/Weight_14"), Icon14x14));

		Set("Sequencer.Actions", new IMAGE_BRUSH_SVG("Starship/Sequencer/Actions", Icon20x20));
		Set("Sequencer.PlaybackOptions", new IMAGE_BRUSH_SVG("Starship/Sequencer/PlaybackOptions", Icon20x20));

		Set("Sequencer.OverlayPanel.Background", new BOX_BRUSH("Sequencer/OverlayPanelBackground", FMargin(26.f / 54.f)));

		Set("Sequencer.TrackArea.LaneColor", FLinearColor(0.3f, 0.3f, 0.3f, 0.3f));

		Set("Sequencer.Tracks.Media", new IMAGE_BRUSH_SVG("Starship/AssetIcons/MediaPlayer_16", Icon16x16));
		Set("Sequencer.Tracks.Audio", new IMAGE_BRUSH_SVG("Starship/AssetIcons/AmbientSound_16", Icon16x16));
		Set("Sequencer.Tracks.Event", new IMAGE_BRUSH_SVG("Starship/Sequencer/EventTrack", Icon16x16));
		Set("Sequencer.Tracks.Fade", new IMAGE_BRUSH_SVG("Starship/Sequencer/FadeTrack", Icon16x16));
		Set("Sequencer.Tracks.CameraCut", new IMAGE_BRUSH_SVG("Starship/Sequencer/CameraCutTrack", Icon16x16));
		Set("Sequencer.Tracks.CinematicShot", new IMAGE_BRUSH_SVG("Starship/Sequencer/ShotTrack", Icon16x16));
		Set("Sequencer.Tracks.Slomo", new IMAGE_BRUSH_SVG("Starship/Sequencer/SlomoTrack", Icon16x16));
		Set("Sequencer.Tracks.TimeWarp", new IMAGE_BRUSH_SVG("Starship/Sequencer/TimeWarp_16", Icon16x16));
		Set("Sequencer.Tracks.Animation", new IMAGE_BRUSH_SVG("Starship/Sequencer/Animation", Icon16x16));
		Set("Sequencer.Tracks.Sub", new IMAGE_BRUSH_SVG("Starship/Sequencer/SubTrack", Icon16x16));
		Set("Sequencer.Tracks.LevelVisibility", new IMAGE_BRUSH_SVG("Starship/Sequencer/LevelVisibilityTrack", Icon16x16));
		Set("Sequencer.Tracks.DataLayer", new IMAGE_BRUSH_SVG("Starship/Common/DataLayers", Icon16x16));
		Set("Sequencer.Tracks.CVar", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Console", Icon16x16));

		Set("Sequencer.CursorDecorator_MarqueeAdd", new IMAGE_BRUSH("Sequencer/CursorDecorator_MarqueeAdd", Icon16x16));
		Set("Sequencer.CursorDecorator_MarqueeSubtract", new IMAGE_BRUSH("Sequencer/CursorDecorator_MarqueeSubtract", Icon16x16));
		Set("Sequencer.CursorDecorator_Retime", new IMAGE_BRUSH("Sequencer/CursorDecorator_Retime", Icon16x16));
		Set("Sequencer.CursorDecorator_EasingHandle", new IMAGE_BRUSH("Sequencer/CursorDecorator_EasingHandle", Icon16x16));

		Set("Sequencer.ClockSource.Platform", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_ClockSource_Platform_16x", Icon14x14));
		Set("Sequencer.ClockSource.Audio", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_ClockSource_Audio_16x", Icon14x14));
		Set("Sequencer.ClockSource.RelativeTimecode", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_ClockSource_RelativeTimecode_16x", Icon14x14));
		Set("Sequencer.ClockSource.Timecode", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_ClockSource_Timecode_16x", Icon14x14));
		Set("Sequencer.ClockSource.PlayEveryFrame", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_ClockSource_PlayEveryFrame_16x", Icon14x14));
		Set("Sequencer.ClockSource.Custom", new IMAGE_BRUSH("Sequencer/Main_Icons/Icon_ClockSource_Custom_16x", Icon14x14));

		Set("Sequencer.BreadcrumbText", FTextBlockStyle(NormalText).SetFont(FStyleFonts::Get().NormalBold));
		Set("Sequencer.BreadcrumbIcon", new IMAGE_BRUSH("Common/SmallArrowRight", Icon10x10));

		Set("Sequencer.AddKey.Details", new IMAGE_BRUSH("Sequencer/AddKey_Details", FVector2f(11.f, 11.f)));

		Set("Sequencer.KeyedStatus.NotKeyed", new IMAGE_BRUSH_SVG("Sequencer/DetailsKeyUnkeyed", FVector2f(11.f, 11.f), FLinearColor::FromSRGBColor(FColor(192, 192, 192, 127))));
		Set("Sequencer.KeyedStatus.Keyed", new IMAGE_BRUSH_SVG("Sequencer/DetailsKeyKeyed", FVector2f(11.f, 11.f), FLinearColor::FromSRGBColor(FColor(254, 133, 57))));
		Set("Sequencer.KeyedStatus.Animated", new IMAGE_BRUSH_SVG("Sequencer/DetailsKeyAnimated", FVector2f(11.f, 11.f), FLinearColor::FromSRGBColor(FColor(254, 133, 57))));
		Set("Sequencer.KeyedStatus.PartialKey", new IMAGE_BRUSH_SVG("Sequencer/DetailsKeyPartialKey", FVector2f(11.f, 11.f), FLinearColor::FromSRGBColor(FColor(254, 133, 57))));

		const FSplitterStyle OutlinerSplitterStyle = FSplitterStyle()
		.SetHandleNormalBrush(FSlateNoResource())
		.SetHandleHighlightBrush(FSlateNoResource());
		Set("Sequencer.AnimationOutliner.Splitter", OutlinerSplitterStyle);

		Set("Sequencer.HyperlinkSpinBox", FSpinBoxStyle(GetWidgetStyle<FSpinBoxStyle>("SpinBox"))
			.SetTextPadding(FMargin(0.f))
			.SetBackgroundBrush(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0.f, 0.f, 0.f, 3.f / 16.0f), FSlateColor::UseSubduedForeground()))
			.SetHoveredBackgroundBrush(FSlateNoResource())
			.SetInactiveFillBrush(FSlateNoResource())
			.SetActiveFillBrush(FSlateNoResource())
			.SetForegroundColor(FSlateColor::UseSubduedForeground())
			.SetArrowsImage(FSlateNoResource())
		);

		Set("Sequencer.PlayTimeSpinBox", FSpinBoxStyle(GetWidgetStyle<FSpinBoxStyle>("SpinBox"))
			.SetTextPadding(FMargin(0.f))
			.SetBackgroundBrush(FSlateNoResource())
			.SetHoveredBackgroundBrush(FSlateNoResource())
			.SetInactiveFillBrush(FSlateNoResource())
			.SetActiveFillBrush(FSlateNoResource())
			.SetForegroundColor(FSlateColor::UseForeground())
			.SetArrowsImage(FSlateNoResource())
		);

		Set("Sequencer.HyperlinkTextBox", FEditableTextBoxStyle()
			.SetTextStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 9))
			.SetBackgroundImageNormal(FSlateNoResource())
			.SetBackgroundImageHovered(FSlateNoResource())
			.SetBackgroundImageFocused(FSlateNoResource())
			.SetBackgroundImageReadOnly(FSlateNoResource())
			.SetBackgroundColor(FLinearColor::Transparent)
			.SetForegroundColor(FSlateColor::UseSubduedForeground())
		);
		Set("Sequencer.FixedFont", DEFAULT_FONT("Mono", 9));

		Set("Sequencer.RecordSelectedActors", new IMAGE_BRUSH("SequenceRecorder/icon_tab_SequenceRecorder_16x", Icon16x16));

		FComboButtonStyle SequencerSectionComboButton = FComboButtonStyle()
		.SetButtonStyle(
			FButtonStyle()
			.SetNormal(FSlateNoResource())
			.SetHovered(FSlateNoResource())
			.SetPressed(FSlateNoResource())
			.SetNormalPadding(FMargin(0, 0, 0, 0))
			.SetPressedPadding(FMargin(0, 1, 0, 0))
		)
		.SetDownArrowImage(IMAGE_BRUSH("Common/ComboArrow", Icon8x8));
		Set("Sequencer.SectionComboButton", SequencerSectionComboButton);

		Set("Sequencer.CreateEventBinding", new IMAGE_BRUSH("Icons/icon_Blueprint_AddFunction_16px", Icon16x16));
		Set("Sequencer.CreateQuickBinding", new IMAGE_BRUSH("Icons/icon_Blueprint_Node_16x", Icon16x16));
		Set("Sequencer.ClearEventBinding", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Delete_40x", Icon16x16));
		Set("Sequencer.MultipleEvents", new IMAGE_BRUSH("Sequencer/MultipleEvents", Icon16x16));
		Set("Sequencer.UnboundEvent", new IMAGE_BRUSH("Sequencer/UnboundEvent", Icon16x16));

		// Sequencer Blending Iconography
		Set("EMovieSceneBlendType::Absolute", new IMAGE_BRUSH("Sequencer/EMovieSceneBlendType_Absolute", FVector2f(32.f, 16.f)));
		Set("EMovieSceneBlendType::Relative", new IMAGE_BRUSH("Sequencer/EMovieSceneBlendType_Relative", FVector2f(32.f, 16.f)));
		Set("EMovieSceneBlendType::Additive", new IMAGE_BRUSH("Sequencer/EMovieSceneBlendType_Additive", FVector2f(32.f, 16.f)));
		Set("EMovieSceneBlendType::Override", new IMAGE_BRUSH_SVG("Sequencer/EMovieSceneBlendType_Override", FVector2f(32.f, 16.f)));
		Set("EMovieSceneBlendType::AdditiveFromBase", new IMAGE_BRUSH("Sequencer/EMovieSceneBlendType_AdditiveFromBase", FVector2f(32.f, 16.f)));

		Set("Sequencer.TrackIsolate", new IMAGE_BRUSH_SVG("Sequencer/TrackIsolate", Icon16x16));
		Set("Sequencer.TrackHide", new IMAGE_BRUSH_SVG("Sequencer/TrackHide", Icon16x16));
		Set("Sequencer.TrackShow", new IMAGE_BRUSH_SVG("Sequencer/TrackShow", Icon16x16));

		constexpr float CommonPadding = 3;
		Set("Sequencer.FilterAreaMargin", FMargin(2 * CommonPadding, CommonPadding));

		Set("Sequencer.KeyTransform.TranslateLeft", new IMAGE_BRUSH_SVG("Sequencer/MoveKeysLeft_20", Icon20x20));
		Set("Sequencer.KeyTransform.TranslateRight", new IMAGE_BRUSH_SVG("Sequencer/MoveKeysRight_20", Icon20x20));
		Set("Sequencer.KeyTransform.ScaleMultiply", new IMAGE_BRUSH_SVG("Sequencer/MultiplySolid_20", Icon20x20));
		Set("Sequencer.KeyTransform.ScaleDivide", new IMAGE_BRUSH_SVG("Sequencer/DivideSolid_20", Icon20x20));

		Set("Sequencer.SimpleView.ToggleToSimpleView", new IMAGE_BRUSH_SVG("Sequencer/PanelTopOpen", Icon20x20));
		Set("Sequencer.SimpleView.ToggleToFullView", new IMAGE_BRUSH_SVG("Sequencer/PanelBottomOpen", Icon20x20));

		Set("Sequencer.SimpleView.KeyDisplay.SelectedAndPinned", new IMAGE_BRUSH_SVG("Sequencer/SelectedAndPinned", Icon20x20));
		Set("Sequencer.SimpleView.KeyDisplay.Selected", new IMAGE_BRUSH("Icons/GeneralTools/Select_40x", Icon20x20));
		{
			FSlateBrush KeyDisplayAllBrush = *GetBrush("GenericCommands.SelectAll");
			KeyDisplayAllBrush.SetImageSize(Icon20x20);
			Set("Sequencer.SimpleView.KeyDisplay.All", new FSlateBrush(KeyDisplayAllBrush));
		}

		Set("Sequencer.SimpleView.DragHandleLeft", new IMAGE_BRUSH_SVG("Sequencer/DragHandleLeft_12", Icon12x12));
		Set("Sequencer.SimpleView.DragHandleRight", new IMAGE_BRUSH_SVG("Sequencer/DragHandleRight_12", Icon12x12));
		Set("Sequencer.SimpleView.DragHandleBoth", new IMAGE_BRUSH_SVG("Sequencer/DragHandleBoth_16", Icon16x16));

		Set("Sequencer.SimpleView.FrameAllKeys", new IMAGE_BRUSH_SVG("Sequencer/FrameAllKeys_16", Icon16x16));
		Set("Sequencer.SimpleView.ResetViewRange", new IMAGE_BRUSH_SVG("Sequencer/ResetViewRange_16", Icon16x16));
		Set("Sequencer.SimpleView.GoToFrame", new IMAGE_BRUSH_SVG("Sequencer/GoToFrame_16", Icon16x16));
	}


	// Sequence recorder standalone UI
	if (IncludeEditorSpecificStyles())
	{
		Set("SequenceRecorder.TabIcon", new IMAGE_BRUSH_SVG("Starship/Sequencer/SequenceRecorder", Icon16x16));
		Set("SequenceRecorder.Common.RecordAll.Small", new IMAGE_BRUSH("SequenceRecorder/icon_RecordAll_40x", Icon20x20));
		Set("SequenceRecorder.Common.RecordAll", new IMAGE_BRUSH("SequenceRecorder/icon_RecordAll_40x", Icon40x40));
		Set("SequenceRecorder.Common.StopAll.Small", new IMAGE_BRUSH("SequenceRecorder/icon_StopAll_40x", Icon20x20));
		Set("SequenceRecorder.Common.StopAll", new IMAGE_BRUSH("SequenceRecorder/icon_StopAll_40x", Icon40x40));
		Set("SequenceRecorder.Common.AddRecording.Small", new IMAGE_BRUSH("SequenceRecorder/icon_AddRecording_40x", Icon20x20));
		Set("SequenceRecorder.Common.AddRecording", new IMAGE_BRUSH("SequenceRecorder/icon_AddRecording_40x", Icon40x40));
		Set("SequenceRecorder.Common.AddCurrentPlayerRecording.Small", new IMAGE_BRUSH("SequenceRecorder/icon_AddCurrentPlayerRecording_40x", Icon20x20));
		Set("SequenceRecorder.Common.AddCurrentPlayerRecording", new IMAGE_BRUSH("SequenceRecorder/icon_AddCurrentPlayerRecording_40x", Icon40x40));
		Set("SequenceRecorder.Common.RemoveRecording.Small", new IMAGE_BRUSH("SequenceRecorder/icon_RemoveRecording_40x", Icon20x20));
		Set("SequenceRecorder.Common.RemoveRecording", new IMAGE_BRUSH("SequenceRecorder/icon_RemoveRecording_40x", Icon40x40));
		Set("SequenceRecorder.Common.RemoveAllRecordings.Small", new IMAGE_BRUSH("SequenceRecorder/icon_RemoveRecording_40x", Icon20x20));
		Set("SequenceRecorder.Common.RemoveAllRecordings", new IMAGE_BRUSH("SequenceRecorder/icon_RemoveRecording_40x", Icon40x40));
		Set("SequenceRecorder.Common.RecordingActive", new IMAGE_BRUSH("Common/SmallCheckBox_Checked", Icon14x14));
		Set("SequenceRecorder.Common.RecordingInactive", new IMAGE_BRUSH("Common/SmallCheckBox", Icon14x14));
	}

	// Sequencer outliner column UI
	if (IncludeEditorSpecificStyles())
	{
		Set("Sequencer.Column.Mute", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/SequencerMute", Icon14x14));
		Set("Sequencer.Column.Locked", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/SequencerLocked", Icon14x14));
		Set("Sequencer.Column.Solo", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/SequencerSolo", Icon14x14));
		Set("Sequencer.Column.Unpinned", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/SequencerUnpinned", Icon14x14));
		Set("Sequencer.Column.CheckBoxIndeterminate", new CORE_IMAGE_BRUSH_SVG("Starship/CoreWidgets/CheckBox/CheckBoxIndeterminate_14", Icon14x14));
		Set("Sequencer.Column.OutlinerColumnBox", new FSlateRoundedBoxBrush(FStyleColors::Header, 2.0f));
		Set("Sequencer.Indicator.Condition", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/SequencerCondition", Icon12x12));
		Set("Sequencer.Indicator.TimeWarp", new IMAGE_BRUSH_SVG("Sequencer/Column_Widgets/SequencerTimeWarp", Icon12x12));
	}
}

void FStarshipEditorStyle::FStyle::SetupViewportStyles()
{
	SetupNewViewportToolbarStyles();
	SetupOldViewportToolbarStyles();
	SetupLegacyViewportToolbarStyles();

	// Viewport actor preview's pin/unpin and attach/detach buttons
	{
		Set("ViewportActorPreview.Pinned", new IMAGE_BRUSH("Common/PushPin_Down", Icon16x16));
		Set("ViewportActorPreview.Unpinned", new IMAGE_BRUSH("Common/PushPin_Up", Icon16x16));
		Set("VRViewportActorPreview.Pinned", new IMAGE_BRUSH("Common/PushPin_Down_VR", Icon64x64));
		Set("VRViewportActorPreview.Unpinned", new IMAGE_BRUSH("Common/PushPin_Up_VR", Icon64x64));
		Set("VRViewportActorPreview.Attached", new IMAGE_BRUSH("Common/ScreenAttach_VR", Icon64x64));
		Set("VRViewportActorPreview.Detached", new IMAGE_BRUSH("Common/ScreenDetach_VR", Icon64x64));
	}
}

void FStarshipEditorStyle::FStyle::SetupNewViewportToolbarStyles()
{
	FSlateColor ToolbarBackgroundColor = FStyleColors::Dropdown;
	const FSlateRoundedBoxBrush BackgroundBrush(ToolbarBackgroundColor, 5.f, ToolbarBackgroundColor, 1.0f);
	const FSlateRoundedBoxBrush BackgroundBrushStart(ToolbarBackgroundColor, FVector4(5.f, 0.f, 0.f, 5.f), ToolbarBackgroundColor, 1.0f);
	const FSlateRoundedBoxBrush BackgroundBrushMiddle(ToolbarBackgroundColor, FVector4(0.f, 0.f, 0.f, 0.f), ToolbarBackgroundColor, 1.0f);
	const FSlateRoundedBoxBrush BackgroundBrushEnd(ToolbarBackgroundColor, FVector4(0.f, 5.f, 5.f, 0.f), ToolbarBackgroundColor, 1.0f);

	FSlateColor ToolbarHoveredColor = FStyleColors::Hover;
	const FSlateRoundedBoxBrush HoveredButtonBrush(ToolbarHoveredColor, 5.f, ToolbarHoveredColor, 1.0f);
	const FSlateRoundedBoxBrush HoveredButtonBrushStart(ToolbarHoveredColor, FVector4(5.f, 0.f, 0.f, 5.f), ToolbarHoveredColor, 1.0f);
	const FSlateRoundedBoxBrush HoveredButtonBrushMiddle(ToolbarHoveredColor, FVector4(0.f, 0.f, 0.f, 0.f), ToolbarHoveredColor, 1.0f);
	const FSlateRoundedBoxBrush HoveredButtonBrushEnd(ToolbarHoveredColor, FVector4(0.f, 5.f, 5.f, 0.f), ToolbarHoveredColor, 1.0f);

	FSlateColor ToolbarPressedColor = FStyleColors::Header;
	const FSlateRoundedBoxBrush PressedButtonBrush(ToolbarPressedColor, 5.f, ToolbarPressedColor, 1.0f);
	const FSlateRoundedBoxBrush PressedButtonBrushStart(ToolbarPressedColor, FVector4(5.f, 0.f, 0.f, 5.f), ToolbarPressedColor, 1.0f);
	const FSlateRoundedBoxBrush PressedButtonBrushMiddle(ToolbarPressedColor, FVector4(0.f, 0.f, 0.f, 0.f), ToolbarPressedColor, 1.0f);
	const FSlateRoundedBoxBrush PressedButtonBrushEnd(ToolbarPressedColor, FVector4(0.f, 5.f, 5.f, 0.f), ToolbarPressedColor, 1.0f);

	FButtonStyle ButtonStyle;
	{
		ButtonStyle.SetNormal(BackgroundBrush)
			.SetHovered(HoveredButtonBrush)
			.SetPressed(PressedButtonBrush)
			.SetNormalForeground(FStyleColors::ForegroundHeader)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::Hover2)
			.SetNormalPadding(FMargin(6.0f, 4.0f))
			.SetPressedPadding(FMargin(6.0f, 4.0f));
	}
	
	FButtonStyle ButtonStyleStart = FButtonStyle(ButtonStyle)
		.SetNormal(BackgroundBrushStart)
		.SetHovered(HoveredButtonBrushStart)
		.SetPressed(PressedButtonBrushStart);
	
	FButtonStyle ButtonStyleMiddle = FButtonStyle(ButtonStyle)
		.SetNormal(BackgroundBrushMiddle)
		.SetHovered(HoveredButtonBrushMiddle)
		.SetPressed(PressedButtonBrushMiddle);
	
	FButtonStyle ButtonStyleEnd = FButtonStyle(ButtonStyle)
		.SetNormal(BackgroundBrushEnd)
		.SetHovered(HoveredButtonBrushEnd)
		.SetPressed(PressedButtonBrushEnd);
	

	const FToolBarStyle SlimToolBarStyle =
		FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

	FCheckBoxStyle ToggleButtonStyle = FCheckBoxStyle(SlimToolBarStyle.ToggleButton)
		.SetCheckedImage(BackgroundBrush)
		.SetUncheckedImage(BackgroundBrush)
		.SetCheckedHoveredImage(HoveredButtonBrush)
		.SetCheckedPressedImage(PressedButtonBrush)
		.SetUncheckedPressedImage(PressedButtonBrush)
		.SetCheckedForegroundColor(FStyleColors::AccentBlue)
		.SetCheckedHoveredForegroundColor(FStyleColors::AccentBlue)
		.SetCheckedPressedForegroundColor(FStyleColors::AccentBlue)
		.SetPadding(FMargin(6.0f, 4.0f));
	
	FCheckBoxStyle ToggleButtonStyleStart = FCheckBoxStyle(ToggleButtonStyle)
		.SetCheckedImage(BackgroundBrushStart)
		.SetCheckedHoveredImage(HoveredButtonBrushStart)
		.SetCheckedPressedImage(PressedButtonBrushStart)
		.SetUncheckedImage(BackgroundBrushStart)
		.SetUncheckedHoveredImage(HoveredButtonBrushStart)
		.SetUncheckedPressedImage(PressedButtonBrushStart);
	
	FCheckBoxStyle ToggleButtonStyleMiddle = FCheckBoxStyle(ToggleButtonStyle)
		.SetCheckedImage(BackgroundBrushMiddle)
		.SetCheckedHoveredImage(HoveredButtonBrushMiddle)
		.SetCheckedPressedImage(PressedButtonBrushMiddle)
		.SetUncheckedImage(BackgroundBrushMiddle)
		.SetUncheckedHoveredImage(HoveredButtonBrushMiddle)
		.SetUncheckedPressedImage(PressedButtonBrushMiddle);
	
	FCheckBoxStyle ToggleButtonStyleEnd = FCheckBoxStyle(ToggleButtonStyle)
		.SetCheckedImage(BackgroundBrushEnd)
		.SetCheckedHoveredImage(HoveredButtonBrushEnd)
		.SetCheckedPressedImage(PressedButtonBrushEnd)
		.SetUncheckedImage(BackgroundBrushEnd)
		.SetUncheckedHoveredImage(HoveredButtonBrushEnd)
		.SetUncheckedPressedImage(PressedButtonBrushEnd);

	FComboButtonStyle ComboButtonStyle = SlimToolBarStyle.ComboButtonStyle;
	{
		FButtonStyle InnerButtonStyle = FButtonStyle(ButtonStyle).SetNormalPadding(0.f).SetPressedPadding(0.f);

		ComboButtonStyle
			.SetButtonStyle(InnerButtonStyle)
			.SetDownArrowImage(IMAGE_BRUSH_SVG("Starship/EditorViewport/small-chevron-down", FVector2f(6.f, 6.f)))
			.SetDownArrowPadding(FMargin(4.0f, 0.f, 0.f, 0.f));
	}

	FToolBarStyle ViewportToolbarStyle = SlimToolBarStyle;
	{
		FSlateBrush SeparatorBrush;
		SeparatorBrush.DrawAs = ESlateBrushDrawType::Box;
		SeparatorBrush.TintColor = FStyleColors::Hover;
		
		FComboButtonStyle WrapButtonComboStyle = FComboButtonStyle(ComboButtonStyle)
		.SetButtonStyle(FButtonStyle()
			.SetNormal(FSlateColorBrush(FStyleColors::Dropdown))
			.SetHovered(FSlateColorBrush(FStyleColors::Hover))
			.SetPressed(FSlateColorBrush(FStyleColors::Header))
			.SetNormalForeground(FStyleColors::Foreground)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::Hover2)
			.SetNormalPadding(FMargin(6.0f, 4.0f))
			.SetPressedPadding(FMargin(6.0f, 4.0f))
		);

		ViewportToolbarStyle.SetIconSize(Icon16x16)
			.SetButtonStyle(ButtonStyle)
			.SetToggleButtonStyle(ToggleButtonStyle)
			.SetButtonPadding(FMargin(4.f, 0.f))
			.SetComboButtonStyle(ComboButtonStyle)
			.SetComboButtonPadding(FMargin(4.0f, 0.f))
			.SetSeparatorBrush(SeparatorBrush)
			.SetSeparatorThickness(1.0f)
			.SetSeparatorPadding(FMargin(4.0f, 0.f))
			.SetBackgroundPadding(FMargin(4.0f))
			.SetWrapButtonStyle(
				FWrapButtonStyle(SlimToolBarStyle.WrapButtonStyle)
				.SetComboButtonStyle(WrapButtonComboStyle)
				.SetWrapButtonIndex(-1)
				.SetWrapButtonPadding(FMargin(4.f, -4.f, -4.f, -4.f))
				.SetSeparatorBrush(SlimToolBarStyle.SeparatorBrush)
				.SetSeparatorThickness(2.0f)
			)
			.SetRaisedChildrenRightPadding(16.0f);

		Set("ViewportToolbar", ViewportToolbarStyle);
		
		Set("ViewportToolbar.Button.Start", ButtonStyleStart);
		Set("ViewportToolbar.Button.Middle", ButtonStyleMiddle);
		Set("ViewportToolbar.Button.End", ButtonStyleEnd);
		
		Set("ViewportToolbar.ToggleButton.Start", ToggleButtonStyleStart);
		Set("ViewportToolbar.ToggleButton.Middle", ToggleButtonStyleMiddle);
		Set("ViewportToolbar.ToggleButton.End", ToggleButtonStyleEnd);
		
		// Ensure split buttons don't have rounded corners on the inside
		Set("ViewportToolbar.SplitToggleButton", ToggleButtonStyleStart);
		FComboButtonStyle SplitComboButton = ViewportToolbarStyle.ComboButtonStyle;
		SplitComboButton.ButtonStyle = ButtonStyleEnd;
		Set("ViewportToolbar.SplitComboButton", SplitComboButton);
	}

	// Variant with no background
	{
		FToolBarStyle ViewportToolbarNoBackground(ViewportToolbarStyle);
		ViewportToolbarNoBackground.SetBackground(FSlateNoResource());
		ViewportToolbarNoBackground.BackgroundPadding = FMargin(0.f);
		Set("NoBackgroundViewportToolbar", ViewportToolbarNoBackground);
		Set("NoBackgroundViewportToolbar.Button.Start", ButtonStyleStart);
        Set("NoBackgroundViewportToolbar.Button.Middle", ButtonStyleMiddle);
        Set("NoBackgroundViewportToolbar.Button.End", ButtonStyleEnd);
        
        Set("NoBackgroundViewportToolbar.ToggleButton.Start", ToggleButtonStyleStart);
        Set("NoBackgroundViewportToolbar.ToggleButton.Middle", ToggleButtonStyleMiddle);
        Set("NoBackgroundViewportToolbar.ToggleButton.End", ToggleButtonStyleEnd);
        
        // Ensure split buttons don't have rounded corners on the inside
        Set("NoBackgroundViewportToolbar.SplitToggleButton", ToggleButtonStyleStart);
        FComboButtonStyle SplitComboButton = ViewportToolbarStyle.ComboButtonStyle;
        SplitComboButton.ButtonStyle = ButtonStyleEnd;
        Set("NoBackgroundViewportToolbar.SplitComboButton", SplitComboButton);
	}

	// Special styling for top-level raised buttons.
	FToolBarStyle ViewportToolbarRaisedStyle = ViewportToolbarStyle;
	{
		const FSlateRoundedBoxBrush TransparentBrush(FStyleColors::Transparent, 0.f, FStyleColors::Transparent, 0.f);

		const auto& ConvertButtonToRaised = [&TransparentBrush](const FButtonStyle& ButtonStyle)
		{
			return FButtonStyle(ButtonStyle)
				.SetNormal(TransparentBrush)
				.SetNormalForeground(FStyleColors::Foreground)
				.SetNormalPadding(4.0f)
				.SetPressedPadding(4.0f);
		};
	
		FButtonStyle ButtonStyleRaised = ConvertButtonToRaised(ButtonStyle);
		FButtonStyle ButtonStyleRaisedStart = ConvertButtonToRaised(ButtonStyleStart);
		FButtonStyle ButtonStyleRaisedMiddle = ConvertButtonToRaised(ButtonStyleMiddle);
		FButtonStyle ButtonStyleRaisedEnd = ConvertButtonToRaised(ButtonStyleEnd);
		
		const auto ConvertToggleButtonToRaised = [&TransparentBrush](const FCheckBoxStyle& ButtonStyle)
		{
			return FCheckBoxStyle(ButtonStyle)
				.SetPadding(4.0f)
				.SetForegroundColor(FStyleColors::Foreground)
				.SetUncheckedImage(TransparentBrush)
				.SetCheckedImage(TransparentBrush);
		};
		
		FCheckBoxStyle ToggleButtonStyleRaised = ConvertToggleButtonToRaised(ToggleButtonStyle);
		FCheckBoxStyle ToggleButtonStyleRaisedStart = ConvertToggleButtonToRaised(ToggleButtonStyleStart);
		FCheckBoxStyle ToggleButtonStyleRaisedMiddle = ConvertToggleButtonToRaised(ToggleButtonStyleMiddle);
		FCheckBoxStyle ToggleButtonStyleRaisedEnd = ConvertToggleButtonToRaised(ToggleButtonStyleEnd);

		FComboButtonStyle ComboStyleRaised = FComboButtonStyle(ViewportToolbarStyle.ComboButtonStyle)
			.SetDownArrowPadding(FMargin(2.0f, 0, 0, 0));

		FComboButtonStyle SettingsComboStyleRaised = ViewportToolbarStyle.SettingsComboButton;
		{
			SettingsComboStyleRaised.ButtonStyle
				.SetNormalPadding(FMargin(2.f, 0.f))
				.SetPressedPadding(FMargin(2.f, 0.f))
				.SetHovered(HoveredButtonBrush)
				.SetHoveredForeground(FStyleColors::ForegroundHover);
		}

		ViewportToolbarRaisedStyle
			.SetButtonStyle(ButtonStyleRaised)
			.SetToggleButtonStyle(ToggleButtonStyleRaised)
			.SetComboButtonStyle(ComboStyleRaised)
			.SetBlockHovered(BackgroundBrush)
			// The offset gap allows raised buttons to be 6px away from themselves & their top-level parents
			.SetButtonPadding(FMargin(2.f, 0.f, 4.f, 0.f))
			.SetComboButtonPadding(FMargin(2.f, 0.f, 4.f, 0.f))
			.SetSettingsComboButtonStyle(SettingsComboStyleRaised);
		
		Set("ViewportToolbar.Raised", ViewportToolbarRaisedStyle);
		
		Set("ViewportToolbar.Raised.Button.Start", ButtonStyleRaisedStart);
		Set("ViewportToolbar.Raised.Button.Middle", ButtonStyleRaisedMiddle);
		Set("ViewportToolbar.Raised.Button.End", ButtonStyleRaisedEnd);
		
		Set("ViewportToolbar.Raised.ToggleButton.Start", ToggleButtonStyleRaisedStart);
		Set("ViewportToolbar.Raised.ToggleButton.Middle", ToggleButtonStyleRaisedMiddle);
		Set("ViewportToolbar.Raised.ToggleButton.End", ToggleButtonStyleRaisedEnd);
		
		// Ensure split buttons don't have rounded corners on the inside
		Set("ViewportToolbar.Raised.SplitToggleButton", ToggleButtonStyleRaisedStart);
		FComboButtonStyle SplitComboButton = ViewportToolbarRaisedStyle.ComboButtonStyle;
		SplitComboButton.ButtonStyle = ButtonStyleRaisedEnd;
		Set("ViewportToolbar.Raised.SplitComboButton", SplitComboButton);
	}

	// Special style for raised transform buttons
	{
		FToolBarStyle TransformToolsParent = FToolBarStyle(GetWidgetStyle<FToolBarStyle>("ViewportToolbar"))
			// The raised children all have no padding, so the raised gap padding needs to be extended
			// to make the sizing consistent.
			// The design target is 24px from the last raised item to the next top-level item.
			.SetRaisedChildrenRightPadding(20.f);
	
		FToolBarStyle RaisedTransformTools = FToolBarStyle(GetWidgetStyle<FToolBarStyle>("ViewportToolbar.Raised"))
			.SetButtonPadding(0.f)
			.SetComboButtonPadding(0.f);
			
		Set("ViewportToolbar.TransformTools", TransformToolsParent);
		Set("ViewportToolbar.TransformTools.Raised", RaisedTransformTools);

		// Reuse the same styling as the base ViewportToolbar for Raised.SplitToggleButton
		Set("ViewportToolbar.TransformTools.Raised.SplitToggleButton", GetWidgetStyle<FCheckBoxStyle>("ViewportToolbar.Raised.SplitToggleButton"));
	}

	// Special styling for warnings.
	{
		// Use normal styling for non-raised buttons.
		Set("ViewportToolbarWarning", ViewportToolbarStyle);

		{
			FLinearColor WarningColor = FStyleColors::Warning.GetSpecifiedColor();
			const FSlateRoundedBoxBrush WarningBrush(WarningColor, 5.0f, WarningColor, 1.0f);

			FLinearColor WarningHoveredColor = FStyleColors::Warning.GetSpecifiedColor();
			WarningHoveredColor.R = FMath::Min(1.0f, WarningHoveredColor.R * 1.5f);
			WarningHoveredColor.G = FMath::Min(1.0f, WarningHoveredColor.G * 1.5f);
			WarningHoveredColor.B = FMath::Min(1.0f, WarningHoveredColor.B * 1.5f);
			const FSlateRoundedBoxBrush WarningHoveredBrush(WarningHoveredColor, 5.0f, WarningHoveredColor, 1.0f);

			FLinearColor WarningPressedColor = FStyleColors::Warning.GetSpecifiedColor();
			WarningPressedColor.A = .50f;
			const FSlateRoundedBoxBrush WarningPressedBrush(WarningPressedColor, 5.0f, WarningPressedColor, 1.0f);

			const FCheckBoxStyle ToggleButtonStyleRaisedWarning =
				FCheckBoxStyle(ViewportToolbarRaisedStyle.ToggleButton)
					.SetBackgroundImage(WarningBrush)
					.SetBackgroundHoveredImage(WarningHoveredBrush)
					.SetUncheckedImage(WarningBrush)
					.SetUncheckedHoveredImage(WarningHoveredBrush)
					.SetUncheckedPressedImage(WarningPressedBrush)
					.SetCheckedImage(WarningBrush)
					.SetCheckedHoveredImage(WarningHoveredBrush)
					.SetCheckedPressedImage(WarningPressedBrush)
					.SetForegroundColor(FStyleColors::ForegroundInverted)
					.SetHoveredForegroundColor(FStyleColors::ForegroundInverted)
					.SetUndeterminedForegroundColor(FStyleColors::ForegroundInverted)
					.SetPressedForegroundColor(FStyleColors::ForegroundInverted);

			const FButtonStyle ButtonStyleRaisedWarning = FButtonStyle(ViewportToolbarRaisedStyle.ButtonStyle)
															  .SetHovered(WarningHoveredBrush)
															  .SetPressed(WarningPressedBrush)
															  .SetNormal(WarningBrush)
															  .SetNormalForeground(FStyleColors::ForegroundInverted)
															  .SetHoveredForeground(FStyleColors::ForegroundInverted)
															  .SetPressedForeground(FStyleColors::ForegroundInverted);

			const FToolBarStyle NewViewportToolbarTopLevelWarningRaisedStyle =
				FToolBarStyle(ViewportToolbarRaisedStyle)
					.SetToggleButtonStyle(ToggleButtonStyleRaisedWarning)
					.SetButtonStyle(ButtonStyleRaisedWarning);

			Set("ViewportToolbarWarning.Raised", NewViewportToolbarTopLevelWarningRaisedStyle);
		}
	}

	// Special style for the "Viewport Sizing" submenu, with rounded corners on the left.
	{
		const FButtonStyle ViewportSizingButtonStyle =
			FButtonStyle(ButtonStyleStart)
				.SetNormalPadding(FMargin(0.f))
				.SetPressedPadding(FMargin(0.f));

		const FComboButtonStyle ViewportSizingComboButtonStyle =
			FComboButtonStyle(ComboButtonStyle)
				.SetDownArrowPadding(FMargin(5.f, 0.f, 5.f, 0.f))
				.SetDownArrowImage(CORE_IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 20.f)))
				.SetButtonStyle(ViewportSizingButtonStyle);

		const FToolBarStyle ViewportSizingStyle = FToolBarStyle(ViewportToolbarStyle)
			.SetButtonStyle(ViewportSizingButtonStyle)
			.SetComboButtonStyle(ViewportSizingComboButtonStyle)
			.SetComboButtonPadding(FMargin(4.0f, 0, 1.0f, 0))
			.SetSeparatorPadding(FMargin(0.f));

		Set("ViewportToolbarViewportSizingSubmenu", ViewportSizingStyle);

		// A raised button keeps the gray background here, with rounded corners on the right.
		{
			FCheckBoxStyle ToggleButtonStyleRaised = FCheckBoxStyle(ToggleButtonStyleEnd)
				// The icon cannot be perfectly centered, so it needs a 1px shove to the right
				// to match the design spec.
				.SetPadding(FMargin(7.0f, 4.0f, 6.0f, 4.0f));
				
			const FToolBarStyle ViewportSizingRaisedStyle = FToolBarStyle(ViewportToolbarStyle)
				.SetToggleButtonStyle(ToggleButtonStyleRaised)
				.SetButtonPadding(FMargin(0.f, 0.f, 4.f, 0.f))
				.SetComboButtonPadding(FMargin(2.0f, 0.f));

			Set("ViewportToolbarViewportSizingSubmenu.Raised", ViewportSizingRaisedStyle);
		}
	}

	// Viewport Toolbar Icons
	{
		Set("ViewportToolbar.CameraSpeed", new IMAGE_BRUSH_SVG("Starship/Common/CameraSpeed_16", Icon16x16));
		Set("ViewportToolbar.CameraSpeedReset", new IMAGE_BRUSH_SVG("Starship/Common/CameraSpeedReset_16", Icon16x16));
		Set("ViewportToolbar.EV100", new IMAGE_BRUSH_SVG("Starship/Common/EV100_16", Icon16x16));
		Set("ViewportToolbar.ExactCameraView", new IMAGE_BRUSH_SVG("Starship/Common/ExactCameraView_16", Icon16x16));
		Set("ViewportToolbar.Exposure", new IMAGE_BRUSH_SVG("Starship/Common/Exposure_16", Icon16x16));
		Set("ViewportToolbar.FarViewPlane", new IMAGE_BRUSH_SVG("Starship/Common/FarViewPlane_16", Icon16x16));
		Set("ViewportToolbar.FieldOfView", new IMAGE_BRUSH_SVG("Starship/Common/FieldOfView_16", Icon16x16));
		Set("ViewportToolbar.GizmoScale", new IMAGE_BRUSH_SVG("Starship/Common/GizmoScale_16", Icon16x16));
		Set("ViewportToolbar.NearViewPlane", new IMAGE_BRUSH_SVG("Starship/Common/NearViewPlane_16", Icon16x16));
		Set("ViewportToolbar.PreviewSceneSettings", new IMAGE_BRUSH_SVG("Starship/AssetEditors/PreviewSceneSettings_16", Icon16x16));
		Set("ViewportToolbar.SetShowGrid", new IMAGE_BRUSH_SVG("Starship/Common/Grid", Icon16x16));
		Set("ViewportToolbar.Snap", new IMAGE_BRUSH_SVG("Starship/Common/Snap_16", Icon16x16));
		Set("ViewportToolbar.SnapLocation", new IMAGE_BRUSH_SVG("Starship/Common/SnapLocation_16", Icon16x16));
		Set("ViewportToolbar.SnapPlanar", new IMAGE_BRUSH_SVG("Starship/Common/SnapPlanar_16", Icon16x16));
		Set("ViewportToolbar.SnapRotation", new IMAGE_BRUSH_SVG("Starship/Common/SnapRotation_16", Icon16x16));
		Set("ViewportToolbar.SnapScale", new IMAGE_BRUSH_SVG("Starship/Common/SnapScale_16", Icon16x16));
		Set("ViewportToolbar.SpeedScalar", new IMAGE_BRUSH_SVG("Starship/Common/SpeedScalar_16", Icon16x16));
		Set("ViewportToolbar.SurfaceSnap", new IMAGE_BRUSH_SVG("Starship/Common/SurfaceSnap_16", Icon16x16));
		Set("ViewportToolbar.SurfaceSnapRotateToNormal", new IMAGE_BRUSH_SVG("Starship/Common/SurfaceSnapRotateToNormal_16", Icon16x16));
		Set("ViewportToolbar.TransformRotate", new IMAGE_BRUSH_SVG("Starship/Common/TransformRotate_16", Icon16x16));
		Set("ViewportToolbar.TransformScale", new IMAGE_BRUSH_SVG("Starship/Common/TransformScale_16", Icon16x16));
		Set("ViewportToolbar.TransformSelect", new IMAGE_BRUSH_SVG("Starship/Common/TransformSelect_16", Icon16x16));
		Set("ViewportToolbar.TransformTranslate", new IMAGE_BRUSH_SVG("Starship/Common/TransformMove_16", Icon16x16));
		Set("ViewportToolbar.VPCamera", new IMAGE_BRUSH_SVG("Starship/Common/VPCamera_16", Icon16x16));
		Set("ViewportToolbar.VPCineCamera", new IMAGE_BRUSH_SVG("Starship/Common/VPCineCamera_16", Icon16x16));
	}
}

void FStarshipEditorStyle::FStyle::SetupOldViewportToolbarStyles()
{
	FToolBarStyle ViewportToolbarStyle = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

	FMargin ViewportMarginLeft(6.f, 4.f, 3.f, 4.f);
	FMargin ViewportMarginCenter(6.f, 4.f, 3.f, 4.f);
	FMargin ViewportMarginRight(4.f, 4.f, 5.f, 4.f);

	const FCheckBoxStyle ViewportToggleButton = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetCheckedImage(FSlateNoResource())
		.SetCheckedHoveredImage(FSlateNoResource())
		.SetCheckedPressedImage(FSlateNoResource())
		.SetUncheckedImage(FSlateNoResource())
		.SetUncheckedHoveredImage(FSlateNoResource())
		.SetUncheckedPressedImage(FSlateNoResource())
		.SetForegroundColor(FStyleColors::Foreground)
		.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetPressedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedForegroundColor(FStyleColors::ForegroundHover)
		.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
		.SetPadding(0);

	FLinearColor ToolbarBackgroundColor = FStyleColors::Dropdown.GetSpecifiedColor();
	ToolbarBackgroundColor.A = .80f;

	FLinearColor ToolbarPressedColor = FStyleColors::Recessed.GetSpecifiedColor();
	ToolbarPressedColor.A = .80f;

	FSlateRoundedBoxBrush* ViewportGroupBrush = new FSlateRoundedBoxBrush(ToolbarBackgroundColor, 12.f, FLinearColor(0.f,0.f,0.f,.8f), 1.0f);
	Set("EditorViewportToolBar.Group", ViewportGroupBrush);

	FSlateRoundedBoxBrush* ViewportGroupPressedBrush = new FSlateRoundedBoxBrush(ToolbarPressedColor, 12.f, FLinearColor(0.f, 0.f, 0.f, .8f), 1.0f);
	Set("EditorViewportToolBar.Group.Pressed", ViewportGroupPressedBrush);

	FButtonStyle ViewportMenuButton = FButtonStyle()
		.SetNormal(*ViewportGroupBrush)
		.SetHovered(*ViewportGroupBrush)
		.SetPressed(*ViewportGroupPressedBrush)
		.SetNormalForeground(FStyleColors::Foreground)
		.SetHoveredForeground(FStyleColors::ForegroundHover)
		.SetPressedForeground(FStyleColors::ForegroundHover)
		.SetDisabledForeground(FStyleColors::Foreground)
		.SetNormalPadding(FMargin(4.0f, 4.0f, 3.0f, 4.0f))
		.SetPressedPadding(FMargin(4.0f, 4.0f, 3.0f, 4.0f));
	Set("EditorViewportToolBar.Button", ViewportMenuButton);

	{
		FButtonStyle ViewportMenuButtonLeft = FButtonStyle(ViewportMenuButton)
			.SetNormal(		BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::Dropdown))
			.SetHovered(	BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::Hover))
			.SetPressed(	BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::Recessed))
			.SetNormalPadding(ViewportMarginLeft)
			.SetPressedPadding(ViewportMarginLeft);
		Set("EditorViewportToolBar.Button.Start", ViewportMenuButtonLeft);

		FButtonStyle ViewportMenuButtonMiddle = FButtonStyle(ViewportMenuButton)
			.SetNormal(		BOX_BRUSH("Starship/EditorViewport/ToolBarMiddleGroup", 12.f/25.f, FStyleColors::Dropdown))
			.SetHovered(	BOX_BRUSH("Starship/EditorViewport/ToolBarMiddleGroup", 12.f/25.f, FStyleColors::Hover))
			.SetPressed(	BOX_BRUSH("Starship/EditorViewport/ToolBarMiddleGroup", 12.f/25.f, FStyleColors::Recessed))
			.SetNormalPadding(ViewportMarginCenter)
			.SetPressedPadding(ViewportMarginCenter);
		Set("EditorViewportToolBar.Button.Middle", ViewportMenuButtonMiddle);

		FButtonStyle ViewportMenuButtonRight = FButtonStyle(ViewportMenuButton)
			.SetNormal(		BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Dropdown))
			.SetHovered(	BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Hover))
			.SetPressed(	BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Recessed))
			.SetNormalPadding(ViewportMarginRight)
			.SetPressedPadding(ViewportMarginRight);
		Set("EditorViewportToolBar.Button.End", ViewportMenuButtonRight);
	}

	Set("EditorViewportToolBar.StartToolbarImage", new BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::Dropdown));

	{
		const FCheckBoxStyle ViewportMenuToggleLeftButtonStyle = FCheckBoxStyle(ViewportToggleButton)
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(		  BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::Dropdown))
			.SetUncheckedPressedImage(BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::Recessed))
			.SetUncheckedHoveredImage(BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::Hover))
			.SetCheckedHoveredImage(  BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::PrimaryHover))
			.SetCheckedPressedImage(  BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::PrimaryPress))
			.SetCheckedImage(         BOX_BRUSH("Starship/EditorViewport/ToolBarLeftGroup", 12.f/25.f, FStyleColors::Primary))
			.SetPadding(ViewportMarginLeft);
		Set("EditorViewportToolBar.ToggleButton.Start", ViewportMenuToggleLeftButtonStyle);

		const FCheckBoxStyle ViewportMenuToggleMiddleButtonStyle = FCheckBoxStyle(ViewportToggleButton)
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(		  BOX_BRUSH("Starship/EditorViewport/ToolBarMiddleGroup", 12.f/25.f, FStyleColors::Dropdown))
			.SetUncheckedPressedImage(BOX_BRUSH("Starship/EditorViewport/ToolBarMiddleGroup", 12.f/25.f, FStyleColors::Recessed))
			.SetUncheckedHoveredImage(BOX_BRUSH("Starship/EditorViewport/ToolBarMiddleGroup", 12.f/25.f, FStyleColors::Hover))
			.SetCheckedHoveredImage(  BOX_BRUSH("Starship/EditorViewport/ToolBarMiddleGroup", 12.f/25.f, FStyleColors::PrimaryHover))
			.SetCheckedPressedImage(  BOX_BRUSH("Starship/EditorViewport/ToolBarMiddleGroup", 12.f/25.f, FStyleColors::PrimaryPress))
			.SetCheckedImage(         BOX_BRUSH("Starship/EditorViewport/ToolBarMiddleGroup", 12.f/25.f, FStyleColors::Primary))
			.SetPadding(ViewportMarginCenter);
		Set("EditorViewportToolBar.ToggleButton.Middle", ViewportMenuToggleMiddleButtonStyle);

		const FCheckBoxStyle ViewportMenuToggleRightButtonStyle = FCheckBoxStyle(ViewportToggleButton)
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(		  BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Dropdown))
			.SetUncheckedPressedImage(BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Recessed))
			.SetUncheckedHoveredImage(BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Hover))
			.SetCheckedHoveredImage(  BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::PrimaryHover))
			.SetCheckedPressedImage(  BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::PrimaryPress))
			.SetCheckedImage(         BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Primary))
			.SetPadding(ViewportMarginRight);
		Set("EditorViewportToolBar.ToggleButton.End", ViewportMenuToggleRightButtonStyle);
	}

	{
		// We want a background-less version as the ComboMenu has its own unified background
		const FToolBarStyle& SlimCoreToolBarStyle = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		FButtonStyle ComboMenuButtonStyle = FButtonStyle(SlimCoreToolBarStyle.ButtonStyle)
			.SetNormal(BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Dropdown))
			.SetPressed(BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Recessed))
			.SetHovered(BOX_BRUSH("Starship/EditorViewport/ToolBarRightGroup", 12.f/25.f, FStyleColors::Hover))
			.SetNormalPadding(0.0)
			.SetPressedPadding(0.0);

		Set("EditorViewportToolBar.ComboMenu.ButtonStyle", ComboMenuButtonStyle);
		Set("EditorViewportToolBar.ComboMenu.ToggleButton", ViewportToggleButton);
		Set("EditorViewportToolBar.ComboMenu.LabelStyle", SlimCoreToolBarStyle.LabelStyle);
	}

	{
		FCheckBoxStyle MaximizeRestoreButton = FCheckBoxStyle(ViewportToolbarStyle.ToggleButton)
			.SetUncheckedImage(*ViewportGroupBrush)
			.SetUncheckedPressedImage(*ViewportGroupPressedBrush)
			.SetUncheckedHoveredImage(*ViewportGroupBrush)
			.SetCheckedImage(*ViewportGroupBrush)
			.SetCheckedHoveredImage(*ViewportGroupBrush)
			.SetCheckedPressedImage(*ViewportGroupPressedBrush)
			.SetForegroundColor(FStyleColors::Foreground)
			.SetPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedForegroundColor(FStyleColors::Foreground)
			.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetPadding(FMargin(4.0f, 4.0f, 3.0f, 4.0f));

		Set("EditorViewportToolBar.MaximizeRestoreButton", MaximizeRestoreButton);

		Set("EditorViewportToolBar.Heading.Padding", FMargin(4.f));

		// Non-grouped Toggle Button
		FCheckBoxStyle SoloToggleButton = FCheckBoxStyle(ViewportToolbarStyle.ToggleButton)
			.SetUncheckedImage(*ViewportGroupBrush)
			.SetUncheckedPressedImage(*ViewportGroupPressedBrush)
			.SetUncheckedHoveredImage(*ViewportGroupBrush)
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 12.f, FLinearColor(0.f, 0.f, 0.f, .8f), 1.0f))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 12.f, FLinearColor(0.f, 0.f, 0.f, .8f), 1.0f))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryPress, 12.f, FLinearColor(0.f, 0.f, 0.f, .8f), 1.0f))
			.SetForegroundColor(FStyleColors::Foreground)
			.SetPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedForegroundColor(FStyleColors::Foreground)
			.SetCheckedPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetPadding(FMargin(6.0f, 4.0f, 6.0f, 4.0f));

		// SComboBox 
		FComboButtonStyle ViewportComboButton = FComboButtonStyle()
			.SetButtonStyle(ViewportMenuButton)
			.SetContentPadding(ViewportMarginCenter);
	
		ViewportToolbarStyle
			.SetBackground(FSlateNoResource())
			.SetIconSize(Icon16x16)
			.SetBackgroundPadding(FMargin(0.f))
			.SetLabelPadding(FMargin(0.f))
			.SetComboButtonPadding(FMargin(4.f, 0.0f))
			.SetBlockPadding(FMargin(0.0f,0.0f))
			.SetIndentedBlockPadding(FMargin(0.f))
			.SetButtonPadding(FMargin(0.f))
			.SetCheckBoxPadding(FMargin(4.0f, 0.0f))
			.SetComboButtonStyle(ViewportComboButton)
			.SetToggleButtonStyle(SoloToggleButton)
			.SetButtonStyle(ViewportMenuButton)
			.SetSeparatorBrush(FSlateNoResource())
			.SetSeparatorPadding(FMargin(2.0f, 0.0f));
	
		ViewportToolbarStyle.WrapButtonStyle.SetExpandBrush(IMAGE_BRUSH("Icons/toolbar_expand_16x", Icon8x8));
	
		Set("EditorViewportToolBar", ViewportToolbarStyle);
	}

	{
		FButtonStyle ViewportMenuWarningButton = FButtonStyle(ViewportMenuButton)
			.SetNormalForeground(FStyleColors::AccentYellow)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::AccentYellow);

		Set("EditorViewportToolBar.WarningButton", ViewportMenuWarningButton);

		Set("EditorViewportToolBar.Background", new FSlateNoResource());
		Set("EditorViewportToolBar.OptionsDropdown", new IMAGE_BRUSH_SVG("Starship/EditorViewport/menu", Icon16x16));

		Set("EditorViewportToolBar.Font", FStyleFonts::Get().Normal);

		Set("EditorViewportToolBar.MenuButton", FButtonStyle(Button)
			.SetNormal(BOX_BRUSH("Common/SmallRoundedButton", FMargin(7.f / 16.f), FLinearColor(1.f, 1.f, 1.f, 0.75f)))
			.SetHovered(BOX_BRUSH("Common/SmallRoundedButton", FMargin(7.f / 16.f), FLinearColor(1.f, 1.f, 1.f, 1.0f)))
			.SetPressed(BOX_BRUSH("Common/SmallRoundedButton", FMargin(7.f / 16.f)))
		);

		Set("EditorViewportToolBar.MenuDropdown", new IMAGE_BRUSH("Common/ComboArrow", Icon8x8));
		Set("EditorViewportToolBar.Maximize.Normal", new IMAGE_BRUSH_SVG("Starship/EditorViewport/square", Icon16x16));
		Set("EditorViewportToolBar.Maximize.Checked", new IMAGE_BRUSH_SVG("Starship/EditorViewport/quad", Icon16x16));
		Set("EditorViewportToolBar.RestoreFromImmersive.Normal", new IMAGE_BRUSH("Icons/icon_RestoreFromImmersive_16px", Icon16x16));

		FLinearColor ViewportOverlayColor = FStyleColors::Input.GetSpecifiedColor();
		ViewportOverlayColor.A = 0.75f;

		Set("EditorViewport.OverlayBrush", new FSlateRoundedBoxBrush(ViewportOverlayColor, 8.0f, FStyleColors::Dropdown, 1.0f));

		const FSlateColorBrush ActionableListViewBrush = FSlateColorBrush(FLinearColor(0.f, 0.f, 0.f, 0.f));
		
		Set("ActionableMessage.Border", new FSlateRoundedBoxBrush(ToolbarBackgroundColor, 4.0f, FLinearColor(0.f,0.f,0.f,.8f), 1.0f));
		Set("ActionableMessage.Warning", new IMAGE_BRUSH_SVG(TEXT("Starship/EditorViewport/alert-solid"), Icon16x16, FStyleColors::Warning));
		Set("ActionableMessage.Update", new IMAGE_BRUSH_SVG(TEXT("Starship/EditorViewport/update"), Icon16x16));
		Set("ActionableMessage.ListView", FTableViewStyle().SetBackgroundBrush(ActionableListViewBrush));
		Set("ActionableMessage.ListViewRow",FTableRowStyle()
			.SetEvenRowBackgroundBrush(ActionableListViewBrush)
			.SetEvenRowBackgroundHoveredBrush(ActionableListViewBrush)
			.SetOddRowBackgroundBrush(ActionableListViewBrush)
			.SetOddRowBackgroundHoveredBrush(ActionableListViewBrush)
			.SetSelectorFocusedBrush(ActionableListViewBrush)
			.SetActiveBrush(ActionableListViewBrush)
			.SetActiveHoveredBrush(ActionableListViewBrush)
			.SetInactiveBrush(ActionableListViewBrush)
			.SetInactiveHoveredBrush(ActionableListViewBrush)
		);
	}
}

void FStarshipEditorStyle::FStyle::SetupLegacyViewportToolbarStyles()
{
	FToolBarStyle ViewportToolbarStyle =
		FToolBarStyle()
		.SetBackground(BOX_BRUSH("Old/Menu_Background", FMargin(8.0f / 64.0f), FLinearColor::Transparent))
		.SetComboButtonPadding(FMargin(0.f))
		.SetButtonPadding(FMargin(0.f))
		.SetCheckBoxPadding(FMargin(4.f))
		.SetSeparatorBrush(BOX_BRUSH("Old/Button", 8.0f / 32.0f, FLinearColor::Transparent))
		.SetSeparatorPadding(FMargin(1.f, 0.f, 0.f, 0.f))
		.SetIconSize(Icon16x16)
		.SetLabelPadding(FMargin(0.0f, 0.0f, 3.0f, 0.0f))
		.SetEditableTextStyle(FEditableTextBoxStyle(NormalEditableTextBoxStyle).SetFont(DEFAULT_FONT("Regular", 9)))
		.SetIndentedBlockPadding(FMargin(0.f))
		.SetBlockPadding(FMargin(0.f))
		.SetLabelStyle(
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 9))
			.SetColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f))
		);
		
	ViewportToolbarStyle.WrapButtonStyle.SetExpandBrush(IMAGE_BRUSH("Icons/toolbar_expand_16x", Icon8x8));
	
	const FString SmallRoundedButton(TEXT("Common/SmallRoundedButton"));
	const FString SmallRoundedButtonStart(TEXT("Common/SmallRoundedButtonLeft"));
	const FString SmallRoundedButtonMiddle(TEXT("Common/SmallRoundedButtonCentre"));
	const FString SmallRoundedButtonEnd(TEXT("Common/SmallRoundedButtonRight"));

	const FLinearColor NormalColor(1.f, 1.f, 1.f, 0.75f);
	const FLinearColor PressedColor(1.f, 1.f, 1.f, 1.f);

	/* Create style for "LegacyViewportMenu.ToggleButton" ... */
	const FCheckBoxStyle ViewportMenuToggleButtonStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), NormalColor))
		.SetUncheckedPressedImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), PressedColor))
		.SetUncheckedHoveredImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), PressedColor))
		.SetCheckedHoveredImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), SelectionColor_Pressed))
		.SetCheckedPressedImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), SelectionColor_Pressed))
		.SetCheckedImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), SelectionColor_Pressed));
	/* ... and add new style */
	ViewportToolbarStyle.SetToggleButtonStyle(ViewportMenuToggleButtonStyle);

	/* Create style for "LegacyViewportMenu.ToggleButton.Start" ... */
	const FCheckBoxStyle ViewportMenuToggleStartButtonStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), NormalColor))
		.SetUncheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), PressedColor))
		.SetUncheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), PressedColor))
		.SetCheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), SelectionColor_Pressed))
		.SetCheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), SelectionColor_Pressed))
		.SetCheckedImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), SelectionColor_Pressed));
	/* ... and add new style */
	Set("LegacyViewportMenu.ToggleButton.Start", ViewportMenuToggleStartButtonStyle);

	/* Create style for "LegacyViewportMenu.ToggleButton.Middle" ... */
	const FCheckBoxStyle ViewportMenuToggleMiddleButtonStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), NormalColor))
		.SetUncheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), PressedColor))
		.SetUncheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), PressedColor))
		.SetCheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), SelectionColor_Pressed))
		.SetCheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), SelectionColor_Pressed))
		.SetCheckedImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), SelectionColor_Pressed));
	/* ... and add new style */
	Set("LegacyViewportMenu.ToggleButton.Middle", ViewportMenuToggleMiddleButtonStyle);

	/* Create style for "LegacyViewportMenu.ToggleButton.End" ... */
	const FCheckBoxStyle ViewportMenuToggleEndButtonStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), NormalColor))
		.SetUncheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), PressedColor))
		.SetUncheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), PressedColor))
		.SetCheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), SelectionColor_Pressed))
		.SetCheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), SelectionColor_Pressed))
		.SetCheckedImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), SelectionColor_Pressed));
	/* ... and add new style */
	Set("LegacyViewportMenu.ToggleButton.End", ViewportMenuToggleEndButtonStyle);

	const FMargin NormalPadding = FMargin(4.0f, 4.0f, 4.0f, 4.0f);
	const FMargin PressedPadding = FMargin(4.0f, 4.0f, 4.0f, 4.0f);

	const FButtonStyle ViewportMenuButton = FButtonStyle(Button)
	.SetNormal(BOX_BRUSH(*SmallRoundedButton, 7.0f / 16.0f, NormalColor))
	.SetPressed(BOX_BRUSH(*SmallRoundedButton, 7.0f / 16.0f, PressedColor))
	.SetHovered(BOX_BRUSH(*SmallRoundedButton, 7.0f / 16.0f, PressedColor))
	.SetPressedPadding(PressedPadding)
	.SetNormalPadding(NormalPadding);

	ViewportToolbarStyle.SetButtonStyle(ViewportMenuButton);

	Set("LegacyViewportMenu.Button.Start", FButtonStyle(ViewportMenuButton)
		.SetNormal(BOX_BRUSH(*SmallRoundedButtonStart, 7.0f / 16.0f, NormalColor))
		.SetPressed(BOX_BRUSH(*SmallRoundedButtonStart, 7.0f / 16.0f, PressedColor))
		.SetHovered(BOX_BRUSH(*SmallRoundedButtonStart, 7.0f / 16.0f, PressedColor))
	);

	Set("LegacyViewportMenu.Button.Middle", FButtonStyle(ViewportMenuButton)
		.SetNormal(BOX_BRUSH(*SmallRoundedButtonMiddle, 7.0f / 16.0f, NormalColor))
		.SetPressed(BOX_BRUSH(*SmallRoundedButtonMiddle, 7.0f / 16.0f, PressedColor))
		.SetHovered(BOX_BRUSH(*SmallRoundedButtonMiddle, 7.0f / 16.0f, PressedColor))
	);

	Set("LegacyViewportMenu.Button.End", FButtonStyle(ViewportMenuButton)
		.SetNormal(BOX_BRUSH(*SmallRoundedButtonEnd, 7.0f / 16.0f, NormalColor))
		.SetPressed(BOX_BRUSH(*SmallRoundedButtonEnd, 7.0f / 16.0f, PressedColor))
		.SetHovered(BOX_BRUSH(*SmallRoundedButtonEnd, 7.0f / 16.0f, PressedColor))
	);

	Set("LegacyViewportMenu", ViewportToolbarStyle);
}

void FStarshipEditorStyle::FStyle::SetupDepthBarStyles()
{
	FDepthBarStyle DepthBarStyle = FDepthBarStyle()
		.SetBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Panel, 4.0f))
		.SetTrack(FSlateRoundedBoxBrush(FStyleColors::Input, 1.0f))
		.SetSliceNormal(FSlateColorBrush(FStyleColors::Dropdown))
		.SetSliceHovered(FSlateColorBrush(FStyleColors::Hover))
		.SetSliceTopBrush(FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, FVector4(2.0f, 2.0f, 0.0f, 0.0f)))
		.SetSliceTopHovered(FSlateRoundedBoxBrush(FStyleColors::Hover2, FVector4(2.0f, 2.0f, 0.0f, 0.0f)))
		.SetSliceBottom(FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, FVector4(0.0f, 0.0f, 2.0f, 2.0f)))
		.SetSliceBottomHovered(FSlateRoundedBoxBrush(FStyleColors::Hover2, FVector4(0.0f, 0.0f, 2.0f, 2.0f)))
		.SetFarPlaneButtonBrush(IMAGE_BRUSH_SVG("Starship/MainToolbar/landscape", Icon16x16))
		.SetNearPlaneButtonBrush(IMAGE_BRUSH_SVG("Starship/AssetIcons/Camera_16", Icon16x16))
		.SetFarPlaneButtonPadding(FMargin(4.0f, 4.0f, 4.0f, 8.0f))
		.SetNearPlaneButtonPadding(FMargin(4.0f, 8.0f, 4.0f, 0.0f))
	;
	Set("EditorViewport.DepthBar", DepthBarStyle);
		
	FToolBarStyle DepthBarToolbarStyle = GetWidgetStyle<FToolBarStyle>("ViewportToolbar.Raised");
	DepthBarToolbarStyle
		.SetBackgroundPadding(FMargin(0.0f))
		.SetButtonPadding(FMargin(2.0f, 4.0f))
		.SetComboButtonPadding(FMargin(2.0f, 4.0f))
		.SetSeparatorPadding(FMargin(2.0f, 4.0f))
		.SetShowLabels(false)
	;
		
	DepthBarToolbarStyle.ComboButtonStyle
		.SetDownArrowImage(FSlateNoResource())
		.SetDownArrowPadding(FMargin(0.0f))
	;
		
	Set("EditorViewport.DepthBar.Toolbar", DepthBarToolbarStyle);
		
	Set("EditorViewport.DepthBar.Background", new FSlateRoundedBoxBrush(FStyleColors::Panel, 8.0f, COLOR("555555FF"), 1.0f));
	Set("EditorViewport.DepthBar.SelectionHighlight", new FSlateColorBrush(FSlateColor(COLOR("EBA20AFF"))));
}

void FStarshipEditorStyle::FStyle::SetupMenuBarStyles()
{
	// MenuBar
	{
		Set("Menu.Label.Padding", FMargin(0.0f, 0.0f, 0.0f, 0.0f));
		Set("Menu.Label.ContentPadding", FMargin(10.0f, 2.0f));
	}
}

void FStarshipEditorStyle::FStyle::SetupGeneralIcons()
{
	Set("Plus", new IMAGE_BRUSH("Icons/PlusSymbol_12x", Icon12x12));
	Set("Minus", new IMAGE_BRUSH("Icons/MinusSymbol_12x", Icon12x12));
	Set("Cross", new IMAGE_BRUSH("Icons/Cross_12x", Icon12x12));
	Set("ArrowUp", new IMAGE_BRUSH("Icons/ArrowUp_12x", Icon12x12));
	Set("ArrowDown", new IMAGE_BRUSH("Icons/ArrowDown_12x", Icon12x12));
	Set("AssetEditor.SaveThumbnail", new IMAGE_BRUSH_SVG("Starship/AssetEditors/SaveThumbnail", Icon20x20));
	Set("AssetEditor.ToggleShowBounds", new IMAGE_BRUSH_SVG("Starship/Common/SetShowBounds", Icon20x20));
	Set("AssetEditor.Apply", new IMAGE_BRUSH_SVG("Starship/Common/Apply", Icon20x20));
	Set("AssetEditor.Simulate", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20x20));
	Set("AssetEditor.ToggleStats", new IMAGE_BRUSH_SVG("Starship/Common/Statistics", Icon20x20));
	Set("AssetEditor.CompileStatus.Background", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20x20));
	Set("AssetEditor.CompileStatus.Overlay.Unknown", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Unknown_Badge", Icon20x20, FStyleColors::AccentYellow));
	Set("AssetEditor.CompileStatus.Overlay.Warning", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Warning_Badge", Icon20x20, FStyleColors::Warning));
	Set("AssetEditor.CompileStatus.Overlay.Good", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Good_Badge", Icon20x20, FStyleColors::AccentGreen));
	Set("AssetEditor.CompileStatus.Overlay.Error", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Fail_Badge", Icon20x20, FStyleColors::Error));
	Set("AssetEditor.ScreenPercentage", new IMAGE_BRUSH_SVG("Starship/Common/ScreenPercentage", Icon16x16));
	
	Set("Debug", new IMAGE_BRUSH_SVG( "Starship/Common/Debug", Icon16x16 ) );
	Set("Modules", new IMAGE_BRUSH_SVG( "Starship/Common/Modules", Icon16x16 ) );
	Set("Versions", new IMAGE_BRUSH_SVG("Starship/Common/Versions", Icon20x20));

	Set("Icons.TextEditor", new IMAGE_BRUSH_SVG("Starship/Common/TextEditor_16", Icon16x16));
}

void FStarshipEditorStyle::FStyle::SetupWindowStyles()
{
	// Override the core "Brushes.Title" brush in this editor style so we can overwrite the color it when the 
	// EditorStyleSetting.bEnableEditorWindowBackgroundColor is enabled which allows users to customize
	// the title bar area

	// NOTE!  This raw pointer is "owned" by the style once we call Set. Therefore we let 
	// the style destroy the brush rather than calling delete within the Editor Style.

	WindowTitleOverride = new FSlateColorBrush(FStyleColors::Title);
	Set("Brushes.Title", WindowTitleOverride);

	Set("WindowSize.Small", FVector2f(480.f, 284.f));
	Set("WindowSize.Medium", FVector2f(680.f, 492.f));
}

void FStarshipEditorStyle::FStyle::SetupProjectBadgeStyle()
{
	Set("SProjectBadge.Text", FTextBlockStyle(NormalText));
	Set("SProjectBadge.BadgeShape", new BOX_BRUSH("ProjectBadge/Badge", Icon16x16, FMargin(6.0f/16.f)));
	Set("SProjectBadge.BadgePadding", FMargin(32.0f, 6.0f, 32.0f, 7.0f));
}

void FStarshipEditorStyle::FStyle::SetupDockingStyles()
{
	// Use the Docking Styles defined in StarshipCoreStyle
	FGlobalTabmanager::Get()->SetShouldUseMiddleEllipsisForDockTabLabel(GetDefault<UEditorStyleSettings>()->bEnableMiddleEllipsis);
}

void FStarshipEditorStyle::FStyle::SetupTutorialStyles()
{
	// Documentation tooltip defaults
	const FSlateColor HyperlinkColor( FStyleColors::Foreground );
	{
		const FTextBlockStyle DocumentationTooltipText = FTextBlockStyle( NormalText )
			.SetFont( DEFAULT_FONT( "Regular", 9 ) )
			.SetColorAndOpacity( FLinearColor::Black );
		Set("Documentation.SDocumentationTooltip", FTextBlockStyle(DocumentationTooltipText));

		const FTextBlockStyle DocumentationTooltipTextSubdued = FTextBlockStyle( NormalText )
			.SetFont( DEFAULT_FONT( "Regular", 8 ) )
			.SetColorAndOpacity( FLinearColor( 0.1f, 0.1f, 0.1f ) );
		Set("Documentation.SDocumentationTooltipSubdued", FTextBlockStyle(DocumentationTooltipTextSubdued));

		const FTextBlockStyle DocumentationTooltipHyperlinkText = FTextBlockStyle( NormalText )
			.SetFont( DEFAULT_FONT( "Regular", 8 ) )
			.SetColorAndOpacity( HyperlinkColor );
		Set("Documentation.SDocumentationTooltipHyperlinkText", FTextBlockStyle(DocumentationTooltipHyperlinkText));

		const FButtonStyle DocumentationTooltipHyperlinkButton = FButtonStyle()
				.SetNormal(BORDER_BRUSH( "Old/HyperlinkDotted", FMargin(0,0,0,3/16.0f), HyperlinkColor ) )
				.SetPressed(FSlateNoResource())
				.SetHovered(BORDER_BRUSH( "Old/HyperlinkUnderline", FMargin(0,0,0,3/16.0f), HyperlinkColor ) );
		Set("Documentation.SDocumentationTooltipHyperlinkButton", FButtonStyle(DocumentationTooltipHyperlinkButton));
	}


	// Documentation defaults
	const FTextBlockStyle DocumentationText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity( FLinearColor::White )
		.SetFont(DEFAULT_FONT( "Regular", 10 ));
	Set("Documentation.Text", FTextBlockStyle(DocumentationText));

	const FTextBlockStyle DocumentationHyperlinkText = FTextBlockStyle(DocumentationText)
		.SetColorAndOpacity( HyperlinkColor );
	Set("Documentation.Hyperlink.Text", FTextBlockStyle(DocumentationHyperlinkText));

	const FTextBlockStyle DocumentationHeaderText = FTextBlockStyle(NormalText)
		.SetColorAndOpacity( FLinearColor::White )
		.SetFont(DEFAULT_FONT("Black", 16));
	Set("Documentation.Header.Text", FTextBlockStyle(DocumentationHeaderText));

	const FButtonStyle DocumentationHyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH( "Old/HyperlinkDotted", FMargin(0,0,0,3/16.0f), HyperlinkColor ) )
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH( "Old/HyperlinkUnderline", FMargin(0,0,0,3/16.0f), HyperlinkColor ) );
	Set("Documentation.Hyperlink.Button", FButtonStyle(DocumentationHyperlinkButton));

	// Documentation
	{
		Set( "Documentation.Content", FTextBlockStyle(DocumentationText) );

		const FHyperlinkStyle DocumentationHyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(DocumentationHyperlinkButton)
			.SetTextStyle(DocumentationText)
			.SetPadding(FMargin(0.0f));
		Set("Documentation.Hyperlink", DocumentationHyperlink);

		Set("Documentation.Hyperlink.Button", FButtonStyle(DocumentationHyperlinkButton));
		Set("Documentation.Hyperlink.Text",   FTextBlockStyle(DocumentationHyperlinkText));
		Set("Documentation.NumberedContent",  FTextBlockStyle(DocumentationText));
		Set( "Documentation.BoldContent", FTextBlockStyle(DocumentationText)
			.SetTypefaceFontName(TEXT("Bold")));
		Set("Documentation.ItalicContent", FTextBlockStyle(DocumentationText)
			.SetTypefaceFontName(TEXT("Italic")));

		Set("Documentation.Header1", FTextBlockStyle(DocumentationHeaderText)
			.SetColorAndOpacity(FStyleColors::White)
			.SetFontSize(16));

		Set("Documentation.Header2", FTextBlockStyle(DocumentationHeaderText)
			.SetColorAndOpacity(FStyleColors::White)
			.SetFontSize(14));

		Set("Documentation.Header3", FTextBlockStyle(DocumentationHeaderText)
			.SetColorAndOpacity(FStyleColors::White)
			.SetFontSize(12));

		Set( "Documentation.Separator", new FSlateColorBrush(FLinearColor::FromSRGBColor(FColor(59, 59, 59))));
	}
}

void FStarshipEditorStyle::FStyle::SetupPropertyEditorStyles()
	{
	// Property / details Window / PropertyTable 
	{
		Set( "PropertyEditor.Grid.TabIcon", new IMAGE_BRUSH( "Icons/icon_PropertyMatrix_16px", Icon16x16 ) );
		Set( "PropertyEditor.Properties.TabIcon", new IMAGE_BRUSH( "Icons/icon_tab_SelectionDetails_16x", Icon16x16 ) );

		Set( "PropertyEditor.RemoveColumn", new IMAGE_BRUSH( "Common/PushPin_Down", Icon16x16, FColor( 96, 194, 253, 255 ).ReinterpretAsLinear() ) );
		Set( "PropertyEditor.AddColumn", new IMAGE_BRUSH( "Common/PushPin_Up", Icon16x16, FColor( 96, 194, 253, 255 ).ReinterpretAsLinear() ) );

		Set( "PropertyEditor.AddColumnOverlay",	new IMAGE_BRUSH( "Common/TinyChalkArrow", FVector2f( 71.f, 20.f), FColor( 96, 194, 253, 255 ).ReinterpretAsLinear() ) );
		Set( "PropertyEditor.AddColumnMessage", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "BoldCondensedItalic", 10 ) )
			.SetColorAndOpacity(FColor( 96, 194, 253, 255 ).ReinterpretAsLinear())
		);
	

		Set( "PropertyEditor.AssetName.ColorAndOpacity", FLinearColor::White );

		Set("PropertyEditor.AssetThumbnailBorder", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::InputOutline, 1.0f));
		Set("PropertyEditor.AssetThumbnailBorderHovered", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Hover2, 1.0f));
		Set("PropertyEditor.AssetTileItem.DropShadow", new BOX_BRUSH("Starship/ContentBrowser/drop-shadow", FMargin(4.0f / 64.0f)));

		Set( "PropertyEditor.AssetClass", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 10 ) )
			.SetColorAndOpacity( FLinearColor::White )
			.SetShadowOffset( FVector2f::UnitVector )
			.SetShadowColorAndOpacity( FLinearColor::Black )
		);

		const FButtonStyle AssetComboStyle = FButtonStyle()
			.SetNormal( BOX_BRUSH( "Common/ButtonHoverHint", FMargin(4.f /16.0f), FLinearColor(1.f,1.f,1.f,0.15f) ) )
			.SetHovered( BOX_BRUSH( "Common/ButtonHoverHint", FMargin(4.f /16.0f), FLinearColor(1.f,1.f,1.f,0.25f) ) )
			.SetPressed( BOX_BRUSH( "Common/ButtonHoverHint", FMargin(4.f /16.0f), FLinearColor(1.f,1.f,1.f,0.30f) ) )
			.SetNormalPadding( FMargin(0.f,0.f,0.f,1.f) )
			.SetPressedPadding( FMargin(0.f,1.f,0.f,0.f) );
		Set( "PropertyEditor.AssetComboStyle", AssetComboStyle );

		Set( "PropertyEditor.HorizontalDottedLine",		new IMAGE_BRUSH( "Common/HorizontalDottedLine_16x1px", FVector2f(16.0f, 1.0f), FLinearColor::White, ESlateBrushTileType::Horizontal ) );
		Set( "PropertyEditor.VerticalDottedLine",		new IMAGE_BRUSH( "Common/VerticalDottedLine_1x16px", FVector2f(1.0f, 16.0f), FLinearColor::White, ESlateBrushTileType::Vertical ) );
		Set( "PropertyEditor.SlateBrushPreview",		new BOX_BRUSH( "PropertyView/SlateBrushPreview_32px", Icon32x32, FMargin(3.f/32.f, 3.f/32.f, 15.f/32.f, 13.f/32.f) ) );

		Set( "PropertyTable.TableRow", GetWidgetStyle<FTableRowStyle>("TableView.AlternatingRow"));
		Set( "PropertyTable.HeaderRow", GetWidgetStyle<FHeaderRowStyle>("TableView.Header"));

		FWindowStyle InViewportDecoratorWindow = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FWindowStyle>("Window");
		InViewportDecoratorWindow.SetCornerRadius(4);

		Set("InViewportDecoratorWindow", InViewportDecoratorWindow);
		FLinearColor TransparentBackground = FStyleColors::Background.GetSpecifiedColor();
		TransparentBackground.A = 0.8f;
		Set("PropertyTable.InViewport.Header", new FSlateRoundedBoxBrush(FStyleColors::Title, FVector4(4.0f, 4.0f, 0.0f, 0.0f)));
		Set("PropertyTable.InViewport.Background", new FSlateRoundedBoxBrush(FSlateColor(TransparentBackground), 4.0f));
		// InViewportToolbar
		{
			FToolBarStyle InViewportToolbar = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar");
			InViewportToolbar.SetBackground(FSlateColorBrush(FStyleColors::Panel));
			InViewportToolbar.SetBackgroundPadding(FMargin(4.0f, 0.0f));
			InViewportToolbar.SetButtonPadding(0.0f);
			InViewportToolbar.SetIconSize(Icon16x16);
			InViewportToolbar.ButtonStyle.SetNormalPadding(FMargin(4, 4, 4, 4));
			InViewportToolbar.ButtonStyle.SetPressedPadding(FMargin(4, 5, 4, 3));
			Set("InViewportToolbar", InViewportToolbar);
		}
		const FTableViewStyle InViewportViewStyle = FTableViewStyle()
			.SetBackgroundBrush(FSlateNoResource());
		Set("PropertyTable.InViewport.ListView", InViewportViewStyle);

		Set("PropertyTable.InViewport.Row", FTableRowStyle(NormalTableRowStyle)
			.SetEvenRowBackgroundBrush(FSlateNoResource())
			.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
			.SetOddRowBackgroundBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(FSlateNoResource())
			.SetSelectorFocusedBrush(FSlateNoResource())
			.SetActiveBrush(FSlateNoResource())
			.SetActiveHoveredBrush(FSlateNoResource())
			.SetInactiveBrush(FSlateNoResource())
			.SetInactiveHoveredBrush(FSlateNoResource())
		);

		const FSplitterStyle TransparentSplitterStyle = FSplitterStyle()
			.SetHandleNormalBrush(FSlateNoResource())
			.SetHandleHighlightBrush(FSlateNoResource());
		Set("PropertyTable.InViewport.Splitter", TransparentSplitterStyle);

		float BorderPadding = 0.5f;
		Set( "PropertyTable.CellBorder",	new FSlateRoundedBoxBrush(FStyleColors::Transparent, 0.0f, FStyleColors::Background, BorderPadding) );
		Set( "PropertyTable.CurrentCellBorder",						new FSlateRoundedBoxBrush(FStyleColors::Primary, 0.0f, FStyleColors::White, BorderPadding) );
		Set( "PropertyTable.SelectedCellBorder",						new FSlateRoundedBoxBrush(FStyleColors::Primary, 0.0f, FStyleColors::Background, BorderPadding) );
		Set( "PropertyTable.EditModeCellBorder",						new FSlateRoundedBoxBrush(FStyleColors::Primary, 0.0f, FStyleColors::Background, BorderPadding) );

		Set( "PropertyTable.Selection.Active",						new IMAGE_BRUSH( "Common/Selector", Icon8x8, SelectionColor ) );

		Set( "PropertyTable.HeaderRow.Column.PathDelimiter",		new IMAGE_BRUSH( "Common/SmallArrowRight", Icon10x10 ) );
		Set( "PropertyTable.ColumnBorder",	new BOX_BRUSH( "Common/CellBorder", FMargin(4.f/16.f), FStyleColors::Background )  );
		Set( "PropertyTable.RowHeader.Background",					new BOX_BRUSH( "Old/Menu_Background", FMargin(4.f/64.f) ) );
		Set( "PropertyTable.RowHeader.BackgroundActive",			new BOX_BRUSH( "Old/Menu_Background", FMargin(4.f/64.f), SelectionColor_Inactive ) );
		Set( "PropertyTable.ReadOnlyEditModeCellBorder",			new BORDER_BRUSH( "Common/ReadOnlyEditModeCellBorder", FMargin(6.f/32.f), SelectionColor ) );
		Set( "PropertyTable.ReadOnlyCellBorder",					new BOX_BRUSH( "Common/ReadOnlyCellBorder", FMargin(4.f/16.f), FLinearColor(0.1f, 0.1f, 0.1f, 0.5f) ) );
		Set( "PropertyTable.ReadOnlySelectedCellBorder",			new BOX_BRUSH( "Common/ReadOnlySelectedCellBorder", FMargin(4.f/16.f), FLinearColor(0.0f, 0.0f, 0.0f, 1.0f) ) );
		Set( "PropertyTable.ReadOnlyCurrentCellBorder",				new BOX_BRUSH( "Common/ReadOnlyCurrentCellBorder", FMargin(4.f/16.f), FLinearColor(0.0f, 0.0f, 0.0f, 1.0f) ) );
		Set( "PropertyTable.Cell.DropDown.Background",				new BOX_BRUSH( "Common/GroupBorder", FMargin(4.f/16.f) ) );
		Set( "PropertyTable.ContentBorder",							new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f) ) );	
		Set( "PropertyTable.NormalFont",							DEFAULT_FONT( "Regular", 9 ) );
		Set( "PropertyTable.BoldFont",								DEFAULT_FONT( "Bold", 9 ) );
		Set( "PropertyTable.FilterFont",							DEFAULT_FONT( "Regular", 10 ) );

		const FTableRowStyle PropertyEditorPropertyRowStyle = FTableRowStyle(NormalTableRowStyle)
			.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Panel))
			.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Panel));

		Set( "PropertyWindow.PropertyRow", PropertyEditorPropertyRowStyle);

		Set( "PropertyWindow.FilterSearch", new IMAGE_BRUSH( "Old/FilterSearch", Icon16x16 ) );
		Set( "PropertyWindow.FilterCancel", new IMAGE_BRUSH( "Old/FilterCancel", Icon16x16 ) );
		Set( "PropertyWindow.Favorites_Disabled", new IMAGE_BRUSH( "Icons/EmptyStar_16x", Icon16x16 ) );
		Set( "PropertyWindow.Locked", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/lock", Icon16x16 ) );
		Set( "PropertyWindow.Unlocked", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/lock-unlocked", Icon16x16 ) );
		Set( "PropertyWindow.DiffersFromDefault", new IMAGE_BRUSH_SVG( "Starship/Common/ResetToDefault", Icon16x16) ) ;
		
		Set( "PropertyWindow.NormalFont", FStyleFonts::Get().Small);
		Set( "PropertyWindow.BoldFont",FStyleFonts::Get().SmallBold);
		Set( "PropertyWindow.ItalicFont", DEFAULT_FONT( "Italic", 8 ) );
		Set( "PropertyWindow.FilterFont", DEFAULT_FONT( "Regular", 10 ) );

		FSlateFontInfo MobilityFont = FStyleFonts::Get().Small;
		MobilityFont.LetterSpacing = 100;

		Set("PropertyWindow.MobilityFont", MobilityFont );
		Set("PropertyWindow.MobilityStatic", new IMAGE_BRUSH_SVG("Starship/Common/MobilityStatic", Icon16x16));
		Set("PropertyWindow.MobilityStationary", new IMAGE_BRUSH_SVG("Starship/Common/MobilityStationary", Icon16x16));
		Set("PropertyWindow.MobilityMoveable", new IMAGE_BRUSH_SVG("Starship/Common/MobilityMoveable", Icon16x16));

		Set( "PropertyWindow.NoOverlayColor", new FSlateNoResource() );
		Set( "PropertyWindow.EditConstColor", new FSlateColorBrush( FColor( 152, 152, 152, 80 ) ) );
		Set( "PropertyWindow.FilteredColor", new FSlateColorBrush( FColor( 0, 255, 0, 80 ) ) );
		Set( "PropertyWindow.FilteredEditConstColor", new FSlateColorBrush( FColor( 152, 152, 152, 80 ).ReinterpretAsLinear() * FColor(0,255,0,255).ReinterpretAsLinear() ) );
		Set( "PropertyWindow.CategoryBackground", new BOX_BRUSH( "/PropertyView/CategoryBackground", FMargin(4.f/16.f) ) );
		Set( "PropertyWindow.CategoryForeground", FLinearColor::Black );
		Set( "PropertyWindow.Button_Clear", new IMAGE_BRUSH( "Icons/Cross_12x", Icon12x12 ) );
		Set( "PropertyWindow.Button_Ellipsis", new IMAGE_BRUSH( "Icons/ellipsis_12x", Icon12x12 ) );
		Set( "PropertyWindow.Button_PickAsset", new IMAGE_BRUSH( "Icons/pillarray_12x", Icon12x12 ) );
		Set( "PropertyWindow.Button_PickActor", new IMAGE_BRUSH( "Icons/levels_16x", Icon12x12 ) );

		Set( "PropertyWindow.WindowBorder", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f) ) );

		FInlineEditableTextBlockStyle NameStyle(FCoreStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle"));
		NameStyle.EditableTextBoxStyle.SetFont(DEFAULT_FONT("Regular", 11))
			.SetForegroundColor(FSlateColor(EStyleColor::White));
		NameStyle.TextStyle.SetFont(DEFAULT_FONT("Regular", 11))
			.SetColorAndOpacity(FSlateColor(EStyleColor::White));
		Set( "DetailsView.ConstantTextBlockStyle", NameStyle.TextStyle);
		Set( "DetailsView.NameTextBlockStyle",  NameStyle );

		Set( "DetailsView.NameChangeCommitted", new BOX_BRUSH( "Common/EditableTextSelectionBackground", FMargin(4.f/16.f) ) );
		Set( "DetailsView.HyperlinkStyle", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 8 ) ) );

		FTextBlockStyle BPWarningMessageTextStyle = FTextBlockStyle(NormalText) .SetFont(DEFAULT_FONT("Regular", 8));
		FTextBlockStyle BPWarningMessageHyperlinkTextStyle = FTextBlockStyle(BPWarningMessageTextStyle).SetColorAndOpacity(FLinearColor(0.25f, 0.5f, 1.0f));

		FButtonStyle EditBPHyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor(0.25f, 0.5f, 1.0f)))
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), FLinearColor(0.25f, 0.5f, 1.0f)));

		FHyperlinkStyle BPWarningMessageHyperlinkStyle = FHyperlinkStyle()
			.SetUnderlineStyle(EditBPHyperlinkButton)
			.SetTextStyle(BPWarningMessageHyperlinkTextStyle)
			.SetPadding(FMargin(0.0f));

		Set( "DetailsView.BPMessageHyperlinkStyle", BPWarningMessageHyperlinkStyle );
		Set( "DetailsView.BPMessageTextStyle", BPWarningMessageTextStyle );

		Set( "DetailsView.GroupSection",              new FSlateNoResource());

		Set( "DetailsView.PulldownArrow.Down",        new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-down", Icon16x16, FStyleColors::Foreground)); 
		Set( "DetailsView.PulldownArrow.Down.Hovered",new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-down", Icon16x16, FStyleColors::ForegroundHover)); 
		Set( "DetailsView.PulldownArrow.Up",          new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-up",   Icon16x16, FStyleColors::Foreground)); 
		Set( "DetailsView.PulldownArrow.Up.Hovered",  new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-up",   Icon16x16, FStyleColors::ForegroundHover)); 

		Set( "DetailsView.EditRawProperties",         new CORE_IMAGE_BRUSH_SVG("Starship/Common/layout-spreadsheet",  Icon16x16, FLinearColor::White) );
		Set( "DetailsView.ViewOptions",         	  new CORE_IMAGE_BRUSH_SVG("Starship/Common/settings",  Icon16x16, FLinearColor::White) );
		Set( "DetailsView.EditConfigProperties",      new IMAGE_BRUSH("Icons/icon_PropertyMatrix_16px",  Icon16x16, FLinearColor::White ) );

		Set( "DetailsView.CollapsedCategory",         new FSlateColorBrush(FStyleColors::Header));
		Set( "DetailsView.CollapsedCategory_Hovered", new FSlateColorBrush(FStyleColors::Hover));
		Set( "DetailsView.CategoryTop",               new FSlateColorBrush(FStyleColors::Header));
		
		/****** Styles for rounded corners for the Card style of a Details View ********/

		Set( "DetailsView.CardHeaderTopLeftSideRounded", new FSlateRoundedBoxBrush(FStyleColors::Header, FVector4(4.0f, 0.0f, 0.0f, 0.0f)));
		Set( "DetailsView.CardHeaderLeftSideRounded", new FSlateRoundedBoxBrush(FStyleColors::Header, FVector4(4.0f, 0.0f, 0.0f, 4.0f)));
		Set( "DetailsView.CardHeaderTopRightSideRounded", new FSlateRoundedBoxBrush(FStyleColors::Header, FVector4(0.0f, 4.0f, 0.0f, 0.0f)));
		Set( "DetailsView.CardHeaderRightSideRounded", new FSlateRoundedBoxBrush(FStyleColors::Header, FVector4(0.0f, 4.0f, 4.0f, 0.0f)));
		Set( "DetailsView.CardHeaderTopRounded", new FSlateRoundedBoxBrush(FStyleColors::Header, FVector4(4.0f, 4.0f, 0.0f, 0.0f)));
		Set( "DetailsView.CardHeaderRounded", new FSlateRoundedBoxBrush(FStyleColors::Header, FVector4(4.0f, 4.0f, 4.0f, 4.0f )));
		
		/*******************************************************************************/
		
		// Background images for all the details panels override states
		Set("DetailsView.OverrideUndetermined", new IMAGE_BRUSH_SVG("Starship/Common/QuestionAnswer", Icon16x16, FStyleColors::AccentPurple));
		Set("DetailsView.OverrideHere", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideHere", Icon16x16, FStyleColors::AccentBlue));
		Set("DetailsView.OverrideAdded", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideAdded", Icon16x16, FStyleColors::AccentGreen));
		Set("DetailsView.OverrideNone", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideNone", Icon16x16, FStyleColors::Foreground));
		Set("DetailsView.OverrideRemoved", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideRemoved", Icon16x16, FStyleColors::Error));
		Set("DetailsView.OverrideInside", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideInside", Icon16x16, FStyleColors::AccentBlue));
		Set("DetailsView.OverrideHereInside", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideHereInside", Icon16x16, FStyleColors::AccentBlue));
		Set("DetailsView.OverrideInherited", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideInherited", Icon16x16, FStyleColors::AccentBlue));
		Set("DetailsView.OverrideAlert", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideAlert", Icon16x16, FStyleColors::White));
		Set("DetailsView.OverrideMixed", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideMixed", Icon16x16, FStyleColors::AccentBlue));

		// Hovered images for all the details panel override states
		Set("DetailsView.OverrideHere.Hovered", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideHere", Icon16x16, FStyleColors::White));
		Set("DetailsView.OverrideAdded.Hovered", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideAdded", Icon16x16, FStyleColors::White));
		Set("DetailsView.OverrideNone.Hovered", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideNone", Icon16x16, FStyleColors::White));
		Set("DetailsView.OverrideRemoved.Hovered", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideRemoved", Icon16x16, FStyleColors::White));
		Set("DetailsView.OverrideInside.Hovered", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideInside", Icon16x16, FStyleColors::White));
		Set("DetailsView.OverrideHereInside.Hovered", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideHereInside", Icon16x16, FStyleColors::White));
		Set("DetailsView.OverrideInherited.Hovered", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideInherited", Icon16x16, FStyleColors::White));
		Set("DetailsView.OverrideAlert.Hovered", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideAlert", Icon16x16, FStyleColors::White));
		Set("DetailsView.OverrideMixed.Hovered", new IMAGE_BRUSH_SVG("Starship/DetailsView/DetailsOverrideMixed", Icon16x16, FStyleColors::White));

		
		const FButtonStyle CategoryRowButton = FButtonStyle()
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Header, 0.f))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Input, 0.f))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Header, 0.f))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Header, 0.f))
			.SetNormalPadding(FMargin(2.f, 0.f, 2.f, 0.f))
			.SetPressedPadding(FMargin(2.f, 0.f, 2.f, 0.f));

		FComboButtonStyle CategoryComboButton = FComboButtonStyle(FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FComboButtonStyle>("ComboButton"))
		.SetButtonStyle(CategoryRowButton)
		.SetDownArrowPadding(FMargin(2.f, 5.f, 3.f, 5.f))
		.SetDownArrowImage(CORE_IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 15.f)));
		CategoryComboButton.ButtonStyle = CategoryRowButton;
		Set( "DetailsView.CategoryComboButton", CategoryComboButton);

		Set( "DetailsView.CategoryTop_Hovered",       new FSlateColorBrush(FStyleColors::Hover));
		Set( "DetailsView.CategoryBottom",            new FSlateColorBrush(FStyleColors::Recessed));
		
		// these are not actually displayed as white, see PropertyEditorConstants::GetRowBackgroundColor
		Set( "DetailsView.CategoryMiddle",            new FSlateColorBrush(FStyleColors::White));
		Set( "DetailsView.Highlight",				  new FSlateRoundedBoxBrush(FStyleColors::Transparent, 0.0f, FStyleColors::AccentBlue, 1.0f));

		Set( "DetailsView.PropertyIsFavorite", new IMAGE_BRUSH("PropertyView/Favorites_Enabled", Icon12x12));
		Set( "DetailsView.PropertyIsNotFavorite", new IMAGE_BRUSH("PropertyView/Favorites_Disabled", Icon12x12));
		Set( "DetailsView.NoFavoritesSystem", new IMAGE_BRUSH("PropertyView/NoFavoritesSystem", Icon12x12));

		Set( "DetailsView.Splitter", FSplitterStyle()
			.SetHandleNormalBrush(FSlateColorBrush(FStyleColors::Recessed))                   
			.SetHandleHighlightBrush(FSlateColorBrush(FStyleColors::Recessed))
		);

		Set( "DetailsView.GridLine", new FSlateColorBrush(FStyleColors::Recessed) );
		Set( "DetailsView.SectionButton", FCheckBoxStyle( FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FCheckBoxStyle>("FilterBar.BasicFilterButton")));

		Set( "DetailsView.ChannelToggleButton", FCheckBoxStyle( FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox"))
			.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Input, 4.0f, FStyleColors::DropdownOutline, 1.0f))
			.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Input, 4.0f, FStyleColors::Hover, 1.0f))
			.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::DropdownOutline, 1.0f))
			.SetCheckedForegroundColor(FStyleColors::White)
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f, FStyleColors::DropdownOutline, 1.0f))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f, FStyleColors::Hover, 1.0f))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f, FStyleColors::DropdownOutline, 1.0f))
			.SetPadding(FMargin(16, 4))
		);

		Set( "DetailsView.CategoryFontStyle", FStyleFonts::Get().SmallBold);
		Set( "DetailsView.CategoryTextStyle", FTextBlockStyle(NormalText)
			.SetFont(GetFontStyle("DetailsView.CategoryFontStyle"))
			.SetColorAndOpacity(FStyleColors::ForegroundHeader)
		);

		Set("DetailsView.CategoryTextStyleUpdate", FTextBlockStyle(NormalText)
			.SetFont(FStyleFonts::Get().Small)
			.SetColorAndOpacity(FStyleColors::ForegroundHeader)
			.SetTransformPolicy(ETextTransformPolicy::ToUpper)
		);

		FButtonStyle DetailsExtensionMenuButton = FButtonStyle(FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FButtonStyle>("NoBorder"))
			.SetNormalForeground(FStyleColors::Foreground)
			.SetHoveredForeground(FStyleColors::ForegroundHover)
			.SetPressedForeground(FStyleColors::ForegroundHover)
			.SetDisabledForeground(FStyleColors::Foreground)
			.SetNormalPadding(FMargin(2, 2, 2, 2))
			.SetPressedPadding(FMargin(2, 3, 2, 1));

		Set("DetailsView.ExtensionToolBar.Button", DetailsExtensionMenuButton);

		FToolBarStyle DetailsExtensionToolBarStyle = FToolBarStyle(FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar"))
			.SetButtonStyle(DetailsExtensionMenuButton)
			.SetIconSize(Icon16x16)
			.SetBackground(FSlateNoResource())
			.SetLabelPadding(FMargin(0.f))
			.SetComboButtonPadding(FMargin(0.f))
			.SetBlockPadding(FMargin(0.f))
			.SetIndentedBlockPadding(FMargin(0.f))
			.SetBackgroundPadding(FMargin(0.f))
			.SetButtonPadding(FMargin(0.f))
			.SetCheckBoxPadding(FMargin(0.f))
			.SetSeparatorBrush(FSlateNoResource())
			.SetSeparatorPadding(FMargin(0.f));
		
		DetailsExtensionToolBarStyle.WrapButtonStyle
			.SetExpandBrush(CORE_IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(4.f, 16.f)));

		Set("DetailsView.ExtensionToolBar", DetailsExtensionToolBarStyle);

		Set("DetailsView.ArrayDropShadow", new IMAGE_BRUSH("Common/ArrayDropShadow", FVector2f(32.f,2.f)));

		Set( "DetailsView.TreeView.TableRow", FTableRowStyle()
			.SetEvenRowBackgroundBrush( FSlateNoResource() )
			.SetEvenRowBackgroundHoveredBrush( FSlateNoResource() )
			.SetOddRowBackgroundBrush( FSlateNoResource() )
			.SetOddRowBackgroundHoveredBrush( FSlateNoResource() )
			.SetSelectorFocusedBrush( FSlateNoResource() )
			.SetActiveBrush( FSlateNoResource() )
			.SetActiveHoveredBrush( FSlateNoResource() )
			.SetInactiveBrush( FSlateNoResource() )
			.SetInactiveHoveredBrush( FSlateNoResource() )
			.SetTextColor( DefaultForeground )
			.SetSelectedTextColor( InvertedForeground )
			.SetDropIndicator_Above(BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0.f, 0.f), SelectionColor))
			.SetDropIndicator_Onto(BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), SelectionColor))
			.SetDropIndicator_Below(BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0.f, 0.f, 10.0f / 16.0f), SelectionColor))
			);

		Set("DetailsView.DropZone.Below", new BOX_BRUSH("Common/VerticalBoxDropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), SelectionColor_Subdued));

		FButtonStyle NameAreaButton = FButtonStyle(Button)
			.SetNormalPadding(FMargin(6, 3))
			.SetPressedPadding(FMargin(6, 3));
		Set("DetailsView.NameAreaButton", NameAreaButton);

		Set("DetailsView.NameAreaComboButton", FComboButtonStyle(GetWidgetStyle<FComboButtonStyle>("ComboButton"))
			.SetButtonStyle(NameAreaButton)
			.SetDownArrowPadding(FMargin(4, 0, 0, 0))
			.SetContentPadding(FMargin(4, 0, 0, 0))
		);

	}
	}

void FStarshipEditorStyle::FStyle::SetupProfilerStyle()
{
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	// Profiler
	{
		// Profiler group brushes
		Set( "Profiler.Group.16", new CORE_BOX_BRUSH( "Icons/Profiler/GroupBorder-16Gray", FMargin(4.0f/16.0f) ) );

		// Profiler toolbar icons
		Set( "Profiler.Tab", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/Visualizer", Icon16x16 ) );
		Set( "Profiler.Tab.GraphView", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Graph_View_Tab_16x", Icon16x16 ) );
		Set( "Profiler.Tab.EventGraph", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_OpenEventGraph_32x", Icon16x16 ) );
		Set( "Profiler.Tab.FiltersAndPresets", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Filter_Presets_Tab_16x", Icon16x16 ) );

		// Generic
		Set( "Profiler.LineGraphArea", new CORE_IMAGE_BRUSH( "Old/White", Icon16x16, FLinearColor(1.0f,1.0f,1.0f,0.25f) ) );
		
		// Tooltip hint icon
		Set( "Profiler.Tooltip.HintIcon10", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Custom_Tooltip_12x", Icon12x12 ) );

		// Text styles
		Set( "Profiler.CaptionBold", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Bold", 10 ) )
			.SetColorAndOpacity( FLinearColor::White )
			.SetShadowOffset( FVector2f::UnitVector )
			.SetShadowColorAndOpacity( FLinearColor(0.f,0.f,0.f,0.8f) )
		);

		Set( "Profiler.TooltipBold", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Bold", 8 ) )
			.SetColorAndOpacity( FLinearColor(0.5f,0.5f,0.5f,1.0f) )
			.SetShadowOffset( FVector2f::UnitVector )
			.SetShadowColorAndOpacity( FLinearColor(0.f,0.f,0.f,0.8f) )
		);

		Set( "Profiler.Tooltip", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 8 ) )
			.SetColorAndOpacity( FLinearColor::White )
			.SetShadowOffset( FVector2f::UnitVector )
			.SetShadowColorAndOpacity( FLinearColor(0.f,0.f,0.f,0.8f) )
		);

		// Event graph icons
		Set( "Profiler.EventGraph.SetRoot", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_SetRoot_32x", Icon32x32 ) );
		Set( "Profiler.EventGraph.CullEvents", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Cull_Events_16x", Icon16x16) );
		Set( "Profiler.EventGraph.FilterEvents", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Filter_Events_16x", Icon16x16) );

		Set( "Profiler.EventGraph.SelectStack", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_SelectStack_32x", Icon32x32 ) );

		Set( "Profiler.EventGraph.ExpandAll", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_ExpandAll_32x", Icon32x32 ) );
		Set( "Profiler.EventGraph.CollapseAll", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_CollapseAll_32x", Icon32x32 ) );
		
		Set( "Profiler.EventGraph.ExpandSelection", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_ExpandSelection_32x", Icon32x32 ) );
		Set( "Profiler.EventGraph.CollapseSelection", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_CollapseSelection_32x", Icon32x32 ) );

		Set( "Profiler.EventGraph.ExpandThread", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_ExpandThread_32x", Icon32x32 ) );
		Set( "Profiler.EventGraph.CollapseThread", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_CollapseThread_32x", Icon32x32 ) );

		Set( "Profiler.EventGraph.ExpandHotPath", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_ExpandHotPath_32x", Icon32x32 ) );
		Set( "Profiler.EventGraph.HotPathSmall", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_HotPath_32x", Icon12x12 ) );

		Set( "Profiler.EventGraph.ExpandHotPath16", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_HotPath_32x", Icon16x16 ) );

		Set( "Profiler.EventGraph.GameThread", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_GameThread_32x", Icon32x32 ) );
		Set( "Profiler.EventGraph.RenderThread", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_RenderThread_32x", Icon32x32 ) );
	
		Set( "Profiler.EventGraph.ViewColumn", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_ViewColumn_32x", Icon32x32 ) );
		Set( "Profiler.EventGraph.ResetColumn", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_ResetColumn_32x", Icon32x32 ) );

		Set( "Profiler.EventGraph.HistoryBack", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_History_Back_16x", Icon16x16) );
		Set( "Profiler.EventGraph.HistoryForward", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_History_Fwd_16x", Icon16x16) );

		Set( "Profiler.EventGraph.MaximumIcon", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Max_Event_Graph_16x", Icon16x16) );
		Set( "Profiler.EventGraph.AverageIcon", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Average_Event_Graph_16x", Icon16x16) );

		Set( "Profiler.EventGraph.FlatIcon", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Events_Flat_16x", Icon16x16) );
		Set( "Profiler.EventGraph.FlatCoalescedIcon", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Events_Flat_Coalesced_16x", Icon16x16) );
		Set( "Profiler.EventGraph.HierarchicalIcon", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Events_Hierarchial_16x", Icon16x16) );

		Set( "Profiler.EventGraph.HasCulledEventsSmall", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Has_Culled_Children_12x", Icon12x12) );
		Set( "Profiler.EventGraph.CulledEvent", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Culled_12x", Icon12x12) );
		Set( "Profiler.EventGraph.FilteredEvent", new CORE_IMAGE_BRUSH( "Icons/Profiler/Profiler_Filtered_12x", Icon12x12) );

		Set( "Profiler.EventGraph.DarkText", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 8 ) )
			.SetColorAndOpacity( FLinearColor::Black )
			.SetShadowOffset( FVector2f::ZeroVector )
			);

		// Thread-view
		Set( "Profiler.ThreadView.SampleBorder", new CORE_BOX_BRUSH( "Icons/Profiler/Profiler_ThreadView_SampleBorder_16x", FMargin( 2.0f / 16.0f ) ) );

		// Event graph selected event border
		Set( "Profiler.EventGraph.Border.TB", new CORE_BOX_BRUSH( "Icons/Profiler/Profiler_Border_TB_16x", FMargin(4.0f/16.0f) ) );
		Set( "Profiler.EventGraph.Border.L", new CORE_BOX_BRUSH( "Icons/Profiler/Profiler_Border_L_16x",   FMargin(4.0f/16.0f) ) );
		Set( "Profiler.EventGraph.Border.R", new CORE_BOX_BRUSH( "Icons/Profiler/Profiler_Border_R_16x",   FMargin(4.0f/16.0f) ) );

		// Misc

		Set( "Profiler.Misc.SortBy", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_SortBy_32x", Icon32x32 ) );
		Set( "Profiler.Misc.SortAscending", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_SortAscending_32x", Icon32x32 ) );
		Set( "Profiler.Misc.SortDescending", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_SortDescending_32x", Icon32x32 ) );

		Set( "Profiler.Misc.ResetToDefault", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_ResetToDefault_32x", Icon32x32 ) );

		Set( "Profiler.Misc.Reset16", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_ResetToDefault_32x", Icon16x16 ) );

		Set( "Profiler.Type.Calls", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_Calls_32x", Icon16x16 ) );
		Set( "Profiler.Type.Event", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_Event_32x", Icon16x16 ) );
		Set( "Profiler.Type.Memory", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_Memory_32x", Icon16x16 ) );
		Set( "Profiler.Type.Number", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_Number_32x", Icon16x16 ) );

		// NumberInt, NumberFloat, Memory, Hierarchical
		Set( "Profiler.Type.NumberInt", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_Number_32x", Icon16x16 ) );
		Set( "Profiler.Type.NumberFloat", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_Number_32x", Icon16x16 ) );
		Set( "Profiler.Type.Memory", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_Memory_32x", Icon16x16 ) );
		Set( "Profiler.Type.Hierarchical", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_Event_32x", Icon16x16 ) );

		Set( "Profiler.Misc.GenericFilter", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_GenericFilter_32x", Icon16x16 ) );
		Set( "Profiler.Misc.GenericGroup", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_GenericGroup_32x", Icon16x16 ) );
		Set( "Profiler.Misc.CopyToClipboard", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_CopyToClipboard_32x", Icon32x32 ) );
	
		Set( "Profiler.Misc.Disconnect", new CORE_IMAGE_BRUSH( "Icons/Profiler/profiler_Disconnect_32x", Icon32x32 ) );

		//Set( "Profiler.Type.Calls", new IMAGE_BRUSH( "Icons/Profiler/profiler_Calls_32x", Icon40x40) );
		//Set( "Profiler.Type.Calls.Small", new IMAGE_BRUSH( "Icons/Profiler/profiler_Calls_32x", Icon20x20) );
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
}
	
void FStarshipEditorStyle::FStyle::SetupGraphEditorStyles()
{
	// Graph Editor
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	{
		Set( "Graph.ForegroundColor", FLinearColor(218.0f/255.0f, 218.0f/255.0f, 218.0f/255.0f, 1.0f) );

		Set( "Graph.TitleBackground", new BOX_BRUSH( "Old/Graph/GraphTitleBackground", FMargin(0.f) ) );
		Set( "Graph.Shadow", new BOX_BRUSH( "Old/Window/WindowBorder", 0.48f ) );
		Set( "Graph.Arrow", new IMAGE_BRUSH( "Old/Graph/Arrow", Icon16x16 ) );
		Set( "Graph.ExecutionBubble", new IMAGE_BRUSH( "Old/Graph/ExecutionBubble", Icon16x16 ) );

		Set( "Graph.PlayInEditor", new BOX_BRUSH( "/Graph/RegularNode_shadow_selected", FMargin(18.0f/64.0f) ) );
		Set( "Graph.ReadOnlyBorder", new BOX_BRUSH( "/Graph/Graph_readonly_border", FMargin(18.0f / 64.0f) ) );

		Set( "Graph.Panel.SolidBackground", new IMAGE_BRUSH( "/Graph/GraphPanel_SolidBackground", Icon16x16, FLinearColor::White, ESlateBrushTileType::Both) );
		Set( "Graph.Panel.GridLineColor",   FLinearColor(0.024f, 0.024f, 0.024f) );
		Set( "Graph.Panel.GridRuleColor",   FLinearColor(0.010f, 0.010f, 0.010f) );
		Set( "Graph.Panel.GridCenterColor", FLinearColor(0.005f, 0.005f, 0.005f) );
		
		Set( "Graph.Panel.GridRulePeriod", 8.0f ); // should be a strictly positive integral value

		Set( "Graph.Node.Separator", new IMAGE_BRUSH( "Old/Graph/NodeVerticalSeparator", Icon8x8 ) );
		Set( "Graph.Node.TitleBackground", new BOX_BRUSH( "Old/Graph/NodeTitleBackground", FMargin(12.0f/64) ) );
		Set( "Graph.Node.NodeBackground", new BOX_BRUSH( "Old/Graph/NodeBackground", FMargin(12.0f/64) ) );

		Set( "Graph.Node.Body", new BOX_BRUSH( "/Graph/RegularNode_body", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f) ) );
		Set( "Graph.Node.TintedBody", new BOX_BRUSH( "/Graph/TintedNode_body", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f) ) );
		Set( "Graph.Node.DisabledBanner", new IMAGE_BRUSH( "/Graph/GraphPanel_StripesBackground", Icon64x64, FLinearColor(0.5f, 0.5f, 0.5f, 0.3f), ESlateBrushTileType::Both ) );
		Set( "Graph.Node.DevelopmentBanner", new IMAGE_BRUSH( "/Graph/GraphPanel_StripesBackground", Icon64x64, FLinearColor::Yellow * FLinearColor(1.f, 1.f, 1.f, 0.3f), ESlateBrushTileType::Both ) );
		Set( "Graph.Node.TitleGloss", new BOX_BRUSH( "/Graph/RegularNode_title_gloss", FMargin(12.0f/64.0f) ) );
		Set( "Graph.Node.ColorSpill", new BOX_BRUSH( "/Graph/RegularNode_color_spill", FMargin(8.0f/64.0f, 3.0f/32.0f, 0.f, 0.f) ) );
		Set( "Graph.Node.TitleHighlight", new BOX_BRUSH( "/Graph/RegularNode_title_highlight", FMargin(16.0f/64.0f, 1.0f, 16.0f/64.0f, 0.0f) ) );
		Set( "Graph.Node.IndicatorOverlay", new IMAGE_BRUSH( "/Graph/IndicatorOverlay_color_spill", FVector2f(128.f, 32.f) ) );

		Set( "Graph.Node.ShadowSize", Icon12x12 );
		Set( "Graph.Node.ShadowSelected", new BOX_BRUSH( "/Graph/RegularNode_shadow_selected", FMargin(18.0f/64.0f) ) );
		Set( "Graph.Node.Shadow", new BOX_BRUSH( "/Graph/RegularNode_shadow", FMargin(18.0f/64.0f) ) );
		
		Set( "Graph.Node.DiffHighlight", new BOX_BRUSH( "/Graph/RegularNode_DiffHighlight", FMargin(18.0f/64.0f) ) );
		Set( "Graph.Node.DiffHighlightShading", new BOX_BRUSH( "/Graph/RegularNode_DiffHighlightShading", FMargin(18.0f/64.0f) ) );

		Set( "Graph.Node.RerouteShadow", new IMAGE_BRUSH( "/Graph/RerouteNode_shadow", Icon64x64 ) );
		Set( "Graph.Node.RerouteShadowSelected", new IMAGE_BRUSH( "/Graph/RerouteNode_shadow_selected", Icon64x64 ) );
		
		Set( "Graph.Node.RerouteDiffHighlight", new BOX_BRUSH( "/Graph/RerouteNode_DiffHighlight", FMargin(18.0f/64.0f) ) );
		Set( "Graph.Node.RerouteDiffHighlightShading", new BOX_BRUSH( "/Graph/RerouteNode_DiffHighlightShading", FMargin(18.0f/64.0f) ) );

		Set( "Graph.CompactNode.ShadowSelected", new BOX_BRUSH( "/Graph/MathNode_shadow_selected", FMargin(18.0f/64.0f) ) );
		
		Set( "Graph.CompactNode.DiffHighlight", new BOX_BRUSH( "/Graph/MathNode_DiffHighlight", FMargin(18.0f/64.0f) ) );
		Set( "Graph.CompactNode.DiffHighlightShading", new BOX_BRUSH( "/Graph/MathNode_DiffHighlightShading", FMargin(18.0f/64.0f) ) );

		Set( "Graph.Node.CommentBubble", new BOX_BRUSH( "Old/Graph/CommentBubble", FMargin(8.f/32.0f) ) );
		Set( "Graph.Node.CommentArrow", new IMAGE_BRUSH( "Old/Graph/CommentBubbleArrow", Icon8x8 ) );
		Set( "Graph.Node.CommentFont", DEFAULT_FONT( "Regular", 10 ) );
		Set( "Graph.Node.Comment.BubbleOffset", FMargin(8.f,0.f,0.f,0.f) );
		Set( "Graph.Node.Comment.PinIconPadding", FMargin(0.f,2.f,0.f,0.f) );
		Set("Graph.Node.Comment.Handle", new IMAGE_BRUSH_SVG("Starship/GraphEditors/Comment_Handle", Icon16x16));
		Set("Graph.Node.Comment.BubblePadding", FVector2f(3.f, 3.f));
		Set("Graph.Node.Comment.BubbleWidgetMargin", FMargin(4.f, 4.f));

		const FCheckBoxStyle CommentTitleButton = FCheckBoxStyle()
			.SetCheckBoxType( ESlateCheckBoxType::CheckBox )
			.SetUncheckedImage( IMAGE_BRUSH( "Icons/icon_Blueprint_CommentBubbleOn_16x", Icon16x16, FLinearColor(1.f, 1.f, 1.f, 0.8f)))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleOn_16x", Icon16x16, FLinearColor(1.f, 1.f, 1.f, 0.9f)))
			.SetUncheckedPressedImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleOn_16x", Icon16x16, FLinearColor(1.f, 1.f, 1.f, 1.f)))
			.SetCheckedImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleOn_16x", Icon16x16, FLinearColor(1.f, 1.f, 1.f, 0.8f)))
			.SetCheckedHoveredImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleOn_16x", Icon16x16, FLinearColor(1.f, 1.f, 1.f, 1.f)))
			.SetCheckedPressedImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleOff_16x", Icon16x16, FLinearColor(1.f, 1.f, 1.f, 0.6f)));
		Set( "CommentTitleButton", CommentTitleButton );

		const FCheckBoxStyle CommentBubbleButton = FCheckBoxStyle()
			.SetCheckBoxType( ESlateCheckBoxType::CheckBox )
			.SetUncheckedImage( IMAGE_BRUSH( "Icons/icon_Blueprint_CommentBubbleOn_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 0.5f)))
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Icons/icon_Blueprint_CommentBubbleOn_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 0.9f)))
			.SetUncheckedPressedImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleOn_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 1.f)))
			.SetCheckedImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleOn_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 0.8f)))
			.SetCheckedHoveredImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleOn_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 1.f)))
			.SetCheckedPressedImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleOn_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 0.6f)));
		Set( "CommentBubbleButton", CommentBubbleButton );

		const FCheckBoxStyle CommentBubblePin = FCheckBoxStyle()
			.SetCheckBoxType( ESlateCheckBoxType::CheckBox )
			.SetUncheckedImage( IMAGE_BRUSH( "Icons/icon_Blueprint_CommentBubbleUnPin_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 0.5f)))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleUnPin_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 0.9f)))
			.SetUncheckedPressedImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubblePin_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 1.f)))
			.SetCheckedImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubblePin_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 0.8f)))
			.SetCheckedHoveredImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubblePin_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 1.f)))
			.SetCheckedPressedImage(IMAGE_BRUSH("Icons/icon_Blueprint_CommentBubbleUnPin_16x", Icon10x10, FLinearColor(1.f, 1.f, 1.f, 0.6f)));
		Set( "CommentBubblePin", CommentBubblePin );


		Set( "Graph.VarNode.Body", new BOX_BRUSH( "/Graph/VarNode_body", FMargin(16.f/64.f, 12.f/28.f) ) );
		Set( "Graph.VarNode.ColorSpill", new IMAGE_BRUSH( "/Graph/VarNode_color_spill", FVector2f(132.f,28.f) ) );
		Set( "Graph.VarNode.Gloss", new BOX_BRUSH( "/Graph/VarNode_gloss", FMargin(16.f/64.f, 16.f/28.f, 16.f/64.f, 4.f/28.f) ) );
		Set( "Graph.VarNode.IndicatorOverlay", new IMAGE_BRUSH("/Graph/IndicatorOverlay_color_spill", FVector2f(64.f, 28.f)));
		
		Set( "Graph.VarNode.ShadowSelected", new BOX_BRUSH( "/Graph/VarNode_shadow_selected", FMargin(26.0f/64.0f) ) );
		Set( "Graph.VarNode.Shadow", new BOX_BRUSH( "/Graph/VarNode_shadow", FMargin(26.0f/64.0f) ) );
		
		Set( "Graph.VarNode.DiffHighlight", new BOX_BRUSH( "/Graph/VarNode_DiffHighlight", FMargin(18.0f/64.0f) ) );
		Set( "Graph.VarNode.DiffHighlightShading", new BOX_BRUSH( "/Graph/VarNode_DiffHighlightShading", FMargin(18.0f/64.0f) ) );

		Set( "Graph.CollapsedNode.Body", new BOX_BRUSH( "/Graph/RegularNode_body", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f) ) );
		Set( "Graph.CollapsedNode.BodyColorSpill", new BOX_BRUSH( "/Graph/CollapsedNode_Body_ColorSpill", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f) ) );

		{
			// State or conduit node
			{
				Set( "Graph.StateNode.Body", new BOX_BRUSH( "/Persona/StateMachineEditor/StateNode_Node_Body", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f) ) );
				Set( "Graph.StateNode.ColorSpill", new BOX_BRUSH( "/Persona/StateMachineEditor/StateNode_Node_ColorSpill", FMargin(4.0f/64.0f, 4.0f/32.0f) ) );

				Set( "Graph.StateNode.Icon", new IMAGE_BRUSH_SVG( "Starship/AnimationBlueprintEditor/AnimationState", Icon16x16 ) );
				Set( "Graph.ConduitNode.Icon", new IMAGE_BRUSH_SVG( "Starship/AnimationBlueprintEditor/AnimationConduit", Icon16x16 ) );
				Set( "Graph.AliasNode.Icon", new IMAGE_BRUSH_SVG( "Starship/AnimationBlueprintEditor/AnimationAlias", Icon16x16 ) );
				Set( "Graph.EntryNode.Icon", new IMAGE_BRUSH_SVG( "Starship/Common/Event", Icon16x16 ) );

				Set( "Graph.StateNode.Pin.BackgroundHovered", new BOX_BRUSH( "/Persona/StateMachineEditor/StateNode_Pin_HoverCue", FMargin(12.0f/64.0f,12.0f/64.0f,12.0f/64.0f,12.0f/64.0f)));
				Set( "Graph.StateNode.Pin.Background", new FSlateNoResource() );

				// Anim state node specific brushes
				Set( "Graph.AnimStateNode.Body", new FSlateRoundedBoxBrush(FLinearColor(0.8f, 0.8f, 0.8f, 1.0f), 7.0f));
				Set( "Graph.AnimStateNode.ColorSpill", new FSlateRoundedBoxBrush(FLinearColor(0.2f, 0.2f, 0.2f, 1.0f), 7.0f));
				Set( "Graph.AnimStateNode.Selection", new FSlateRoundedBoxBrush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 7.0f, FStyleColors::AccentOrange, 3.0f));
				Set( "Graph.AnimAliasNode.Body", new FSlateRoundedBoxBrush(FLinearColor(0.8f, 0.8f, 0.8f, 1.0f)));
				Set( "Graph.AnimAliasNode.ColorSpill", new FSlateRoundedBoxBrush(FLinearColor(0.2f, 0.2f, 0.2f, 1.0f)));
				Set( "Graph.AnimAliasNode.Selection", new FSlateRoundedBoxBrush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), FStyleColors::AccentOrange, 3.0f));
				Set( "Graph.AnimAliasNode.Shadow", new BOX_BRUSH( "/Persona/StateMachineEditor/AliasNode_shadow", FMargin(26.0f/64.0f) ));
				Set( "Graph.AnimConduitNode.Body", new FSlateRoundedBoxBrush(FLinearColor(0.8f, 0.8f, 0.8f, 1.0f), 2.0f));
				Set( "Graph.AnimConduitNode.ColorSpill", new FSlateRoundedBoxBrush(FLinearColor(0.2f, 0.2f, 0.2f, 1.0f), 2.0f));
				Set( "Graph.AnimConduitNode.Selection", new FSlateRoundedBoxBrush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 2.0f, FStyleColors::AccentOrange, 3.0f));
				
				Set( "Graph.AnimStateNode.Pin.BackgroundHovered", new FSlateRoundedBoxBrush(FLinearColor(0.8f, 0.8f, 0.8f, 0.5f), 7.0f));
				Set( "Graph.AnimAliasNode.Pin.BackgroundHovered", new FSlateRoundedBoxBrush(FLinearColor(0.8f, 0.8f, 0.8f, 0.5f)));
				Set( "Graph.AnimConduitNode.Pin.BackgroundHovered", new FSlateRoundedBoxBrush(FLinearColor(0.8f, 0.8f, 0.8f, 0.5f), 2.0f));
				Set( "Graph.AnimStateNode.Pin.Background", new FSlateNoResource() );

				Set( "Graph.AnimStateNode.ConnectionArrow", new IMAGE_BRUSH_SVG( "/Persona/StateMachineEditor/ArrowHead", Icon16x16 ) );
			}

			{
				FTextBlockStyle GraphStateNodeTitle = FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT( "Bold", 14 ) )
					.SetColorAndOpacity( FLinearColor(230.0f/255.0f,230.0f/255.0f,230.0f/255.0f) )
					.SetShadowOffset( FVector2f( 2.f,2.f) )
					.SetShadowColorAndOpacity( FLinearColor(0.f,0.f,0.f, 0.7f) );
				Set( "Graph.StateNode.NodeTitle", GraphStateNodeTitle );

				FEditableTextBoxStyle GraphStateNodeTitleEditableText = FEditableTextBoxStyle()
					.SetTextStyle(NormalText)
					.SetFont(NormalText.Font)
					.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
					.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Hover, CoreStyleConstants::InputFocusThickness))
					.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Primary, CoreStyleConstants::InputFocusThickness))
					.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
					.SetFocusedForegroundColor(FStyleColors::White)
					.SetScrollBarStyle( ScrollBar );
				Set( "Graph.StateNode.NodeTitleEditableText", GraphStateNodeTitleEditableText );

				Set( "Graph.StateNode.NodeTitleInlineEditableText", FInlineEditableTextBlockStyle()
					.SetTextStyle(GraphStateNodeTitle)
					.SetEditableTextBoxStyle(GraphStateNodeTitleEditableText)
					);
			}
			
			{	
				FTextBlockStyle AnimGraphStateNodeTitle = FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT( "Bold", 12 ) )
					.SetColorAndOpacity( FLinearColor(230.0f/255.0f,230.0f/255.0f,230.0f/255.0f) );
				Set( "Graph.AnimStateNode.NodeTitle", AnimGraphStateNodeTitle );

				FEditableTextBoxStyle AnimGraphStateNodeTitleEditableText = FEditableTextBoxStyle()
					.SetTextStyle(AnimGraphStateNodeTitle)
					.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
					.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Hover, CoreStyleConstants::InputFocusThickness))
					.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Primary, CoreStyleConstants::InputFocusThickness))
					.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
					.SetFocusedForegroundColor(FStyleColors::White)
					.SetScrollBarStyle( ScrollBar );
				Set( "Graph.AnimStateNode.NodeTitleEditableText", AnimGraphStateNodeTitle );

				Set( "Graph.AnimStateNode.NodeTitleInlineEditableText", FInlineEditableTextBlockStyle()
					.SetTextStyle(AnimGraphStateNodeTitle)
					.SetEditableTextBoxStyle(AnimGraphStateNodeTitleEditableText)
					);
			}

			// Transition nodes
			{
				FMargin TestMargin(16.f/64.f, 16.f/28.f, 16.f/64.f, 4.f/28.f);
				Set( "Graph.TransitionNode.ColorSpill", new BOX_BRUSH( "/Persona/StateMachineEditor/Trans_Node_ColorSpill", TestMargin ) );
				Set( "Graph.TransitionNode.Icon", new IMAGE_BRUSH( "/Persona/StateMachineEditor/Trans_Node_Icon", FVector2f(25.f,25.f) ) );
				Set( "Graph.TransitionNode.Icon_Inertialization", new IMAGE_BRUSH( "/Persona/StateMachineEditor/Trans_Node_Icon_Inertialization", FVector2f(25.f,25.f) ) );

				Set( "Graph.AnimTransitionNode.ColorSpill", new FSlateRoundedBoxBrush(FLinearColor(0.8f, 0.8f, 0.8f, 1.0f)));
				Set( "Graph.AnimTransitionNode.Selection", new FSlateRoundedBoxBrush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), FStyleColors::AccentOrange, 3.0f));
				Set( "Graph.AnimTransitionNode.Icon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-right", FVector2f(25.f, 25.f)));
				Set( "Graph.AnimTransitionNode.Icon_Inertialization", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-right", FVector2f(25.f, 25.f)));
				Set( "Graph.AnimTransitionNode.Shadow", new BOX_BRUSH( "/Persona/StateMachineEditor/TransitionNode_shadow", FMargin(26.0f/64.0f) ) );
			}

			// Transition rule tooltip name 
			{
				Set( "Graph.TransitionNode.TooltipName", FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT( "Bold", 12 ) )
					.SetColorAndOpacity( FLinearColor(218.0f/255.0f,218.0f/255.0f,218.0f/255.0f) )
					.SetShadowOffset( FVector2f::UnitVector )
					.SetShadowColorAndOpacity( FLinearColor(0.f,0.f,0.f, 0.7f) )
				);
			}

			// Transition rule tooltip caption
			{
				Set( "Graph.TransitionNode.TooltipRule", FTextBlockStyle(NormalText)
					.SetFont( DEFAULT_FONT( "Bold", 8 ) )
					.SetColorAndOpacity( FLinearColor(180.0f/255.0f,180.0f/255.0f,180.0f/255.0f) )
					.SetShadowOffset( FVector2f::UnitVector )
					.SetShadowColorAndOpacity( FLinearColor(0.f,0.f,0.f, 0.7f) )
				);
			}

			Set( "Persona.RetargetManager.BoldFont",							DEFAULT_FONT( "Bold", 12 ) );
			Set( "Persona.RetargetManager.SmallBoldFont",						DEFAULT_FONT( "Bold", 10 ) );
			Set( "Persona.RetargetManager.FilterFont",							DEFAULT_FONT( "Regular", 10 ) );
			Set( "Persona.RetargetManager.ItalicFont",							DEFAULT_FONT( "Italic", 9 ) );

			Set("Persona.RetargetManager.ImportantText", FTextBlockStyle(NormalText)
				.SetFont(DEFAULT_FONT("Bold", 11))
				.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f, 1.0f))
				.SetShadowOffset(FVector2f::UnitVector)
				.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.9f)));
		}

		// Behavior Tree Editor
		{
			Set( "BTEditor.Graph.BTNode.Body", new BOX_BRUSH( "/BehaviorTree/BTNode_ColorSpill", FMargin(16.f/64.f, 25.f/64.f, 16.f/64.f, 16.f/64.f) ) );
			Set( "BTEditor.Graph.BTNode.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Run_Behaviour_24x", Icon16x16 ) );

			Set( "BTEditor.Graph.BTNode.Root.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Run_Behaviour_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Composite.Selector.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Selector_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Composite.Sequence.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Sequence_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Composite.SimpleParallel.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Simple_Parallel_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.Blackboard.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Blackboard_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.CompareBlackboardEntries.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Compare_Blackboard_Entries_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.Conditional.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Conditional_Decorator_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.ConeCheck.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Cone_Check_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.Cooldown.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Cooldown_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.DoesPathExist.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Does_Path_Exist_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.ForceSuccess.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Force_Success_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.KeepInCone.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Keep_In_Cone_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.Loop.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Loop_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.NonConditional.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Non_Conditional_Decorator_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.Optional.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Optional_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.ReachedMoveGoal.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Reached_Move_Goal_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Decorator.TimeLimit.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Time_Limit_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Service.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Service_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Service.DefaultFocus.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Default_Focus_Service_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Task.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Task_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Task.MakeNoise.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Make_Noise_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Task.MoveDirectlyToward.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Move_Directly_Toward_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Task.MoveTo.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Move_To_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Task.PlaySound.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Play_Sound_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Task.RunBehavior.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Run_Behaviour_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Task.RunEQSQuery.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/EQS_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Task.Wait.Icon", new IMAGE_BRUSH( "/BehaviorTree/Icons/Wait_24x", Icon24x24 ) );
			Set( "BTEditor.Graph.BTNode.Blueprint", new IMAGE_BRUSH( "/BehaviorTree/Icons/Blueprint_Referencer_16x", Icon16x16 ) );
			Set( "BTEditor.Graph.BTNode.Index", new BOX_BRUSH( "/BehaviorTree/IndexCircle", Icon20x20, FMargin(9.0f/20.0f, 1.0f/20.0f, 9.0f/20.0f, 3.0f/20.0f) ) );

			Set( "BTEditor.Graph.BTNode.Index.Color", FLinearColor(0.3f, 0.3f, 0.3f, 1.0f) );
			Set( "BTEditor.Graph.BTNode.Index.HoveredColor", FLinearColor(1.0f, 0.0f, 0.0f, 1.0f) );
			

			FTextBlockStyle GraphNodeTitle = FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Bold", 8 ) );
			Set( "BTEditor.Graph.BTNode.IndexText", GraphNodeTitle );

			Set( "BTEditor.Debugger.BackOver", new IMAGE_BRUSH( "Icons/icon_step_back_40x", Icon40x40 ) );
			Set( "BTEditor.Debugger.BackOver.Small", new IMAGE_BRUSH( "Icons/icon_step_back_40x", Icon20x20 ) );
			Set( "BTEditor.Debugger.BackInto", new IMAGE_BRUSH("Icons/icon_step_back_40x", Icon40x40));
			Set( "BTEditor.Debugger.BackInto.Small", new IMAGE_BRUSH("Icons/icon_step_back_40x", Icon20x20));
			Set( "BTEditor.Debugger.ForwardInto", new IMAGE_BRUSH("Icons/icon_step_40x", Icon40x40));
			Set( "BTEditor.Debugger.ForwardInto.Small", new IMAGE_BRUSH( "Icons/icon_step_40x", Icon20x20 ) );
			Set( "BTEditor.Debugger.ForwardOver", new IMAGE_BRUSH("Icons/icon_step_40x", Icon40x40));
			Set( "BTEditor.Debugger.ForwardOver.Small", new IMAGE_BRUSH("Icons/icon_step_40x", Icon20x20));
			Set( "BTEditor.Debugger.StepOut", new IMAGE_BRUSH("Icons/icon_step_40x", Icon40x40));
			Set( "BTEditor.Debugger.StepOut.Small", new IMAGE_BRUSH("Icons/icon_step_40x", Icon20x20));
			Set( "BTEditor.Debugger.SingleStep", new IMAGE_BRUSH("Icons/icon_advance_40x", Icon40x40));
			Set( "BTEditor.Debugger.SingleStep.Small", new IMAGE_BRUSH( "Icons/icon_advance_40x", Icon20x20 ) );
			Set( "BTEditor.Debugger.OpenParentTree", new IMAGE_BRUSH("Icons/icon_DebugStepOut_40x", Icon40x40));
			Set( "BTEditor.Debugger.OpenParentTree.Small", new IMAGE_BRUSH("Icons/icon_DebugStepOut_40x", Icon20x20));
			Set( "BTEditor.Debugger.OpenSubtree", new IMAGE_BRUSH("Icons/icon_DebugStepIn_40x", Icon40x40));
			Set( "BTEditor.Debugger.OpenSubtree.Small", new IMAGE_BRUSH("Icons/icon_DebugStepIn_40x", Icon20x20));

			Set( "BTEditor.Debugger.PausePlaySession", new IMAGE_BRUSH( "Icons/icon_pause_40x", Icon40x40 ) );
			Set( "BTEditor.Debugger.PausePlaySession.Small", new IMAGE_BRUSH( "Icons/icon_pause_40x", Icon20x20 ) );
			Set("BTEditor.Debugger.ResumePlaySession", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon40x40));
			Set("BTEditor.Debugger.ResumePlaySession.Small", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20x20));
			Set( "BTEditor.Debugger.StopPlaySession", new IMAGE_BRUSH( "Icons/icon_stop_40x", Icon40x40 ) );
			Set( "BTEditor.Debugger.StopPlaySession.Small", new IMAGE_BRUSH( "Icons/icon_stop_40x", Icon20x20 ) );
			Set("BTEditor.Debugger.LateJoinSession", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon40x40));
			Set("BTEditor.Debugger.LateJoinSession.Small", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20x20));

			Set( "BTEditor.Debugger.CurrentValues", new IMAGE_BRUSH( "BehaviorTree/Debugger_Current_40x", Icon40x40 ) );
			Set( "BTEditor.Debugger.CurrentValues.Small", new IMAGE_BRUSH( "BehaviorTree/Debugger_Current_40x", Icon20x20 ) );
			Set( "BTEditor.Debugger.SavedValues", new IMAGE_BRUSH( "BehaviorTree/Debugger_Saved_40x", Icon40x40 ) );
			Set( "BTEditor.Debugger.SavedValues.Small", new IMAGE_BRUSH( "BehaviorTree/Debugger_Saved_40x", Icon20x20 ) );

			Set( "BTEditor.DebuggerOverlay.Breakpoint.Disabled", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Breakpoint_Disabled", Icon32x32 ) );
			Set( "BTEditor.DebuggerOverlay.Breakpoint.Enabled", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Breakpoint_Valid", Icon32x32 ) );
			Set( "BTEditor.DebuggerOverlay.ActiveNodePointer", new IMAGE_BRUSH( "Old/Kismet2/IP_Normal", FVector2f(128.f,96.f)) );
			Set( "BTEditor.DebuggerOverlay.SearchTriggerPointer", new IMAGE_BRUSH( "/BehaviorTree/SearchTriggerPointer", FVector2f(48.f,64.f)) );
			Set( "BTEditor.DebuggerOverlay.FailedTriggerPointer", new IMAGE_BRUSH( "/BehaviorTree/FailedTriggerPointer", FVector2f(48.f,64.f)) );
			Set( "BTEditor.DebuggerOverlay.BreakOnBreakpointPointer", new IMAGE_BRUSH( "Old/Kismet2/IP_Breakpoint", FVector2f(128.f,96.f)) );

			Set( "BTEditor.Blackboard.NewEntry", new IMAGE_BRUSH_SVG( "Starship/BehaviorTree/BlackboardNewKey", Icon20x20 ) );

			Set( "BTEditor.SwitchToBehaviorTreeMode", new IMAGE_BRUSH_SVG( "Starship/BehaviorTree/BehaviorTree_20", Icon20x20));
			Set( "BTEditor.SwitchToBlackboardMode", new IMAGE_BRUSH_SVG( "Starship/BehaviorTree/Blackboard_20", Icon20x20));

			// Blackboard classes
			Set( "ClassIcon.BlackboardKeyType_Bool", new FSlateRoundedBoxBrush( FLinearColor(0.300000f, 0.0f, 0.0f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_Class", new FSlateRoundedBoxBrush( FLinearColor(0.1f, 0.0f, 0.5f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_Enum", new FSlateRoundedBoxBrush( FLinearColor(0.0f, 0.160000f, 0.131270f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_Float", new FSlateRoundedBoxBrush( FLinearColor(0.357667f, 1.0f, 0.060000f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_Int", new FSlateRoundedBoxBrush( FLinearColor(0.013575f, 0.770000f, 0.429609f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_Name", new FSlateRoundedBoxBrush( FLinearColor(0.607717f, 0.224984f, 1.0f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_NativeEnum", new FSlateRoundedBoxBrush( FLinearColor(0.0f, 0.160000f, 0.131270f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_Object", new FSlateRoundedBoxBrush( FLinearColor(0.0f, 0.4f, 0.910000f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_Rotator", new FSlateRoundedBoxBrush( FLinearColor(0.353393f, 0.454175f, 1.0f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_String", new FSlateRoundedBoxBrush( FLinearColor(1.0f, 0.0f, 0.660537f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );
			Set( "ClassIcon.BlackboardKeyType_Struct", new FSlateRoundedBoxBrush( FLinearColor(0.f, 0.349019f, 0.796070f, 1.0f), 2.5f, FVector2f(16.f, 5.f)));
			Set( "ClassIcon.BlackboardKeyType_Vector", new FSlateRoundedBoxBrush( FLinearColor(1.0f, 0.591255f, 0.016512f, 1.0f), 2.5f, FVector2f(16.f, 5.f) ) );

			Set( "BTEditor.Common.NewBlackboard", new IMAGE_BRUSH_SVG( "Starship/BehaviorTree/Blackboard_20", Icon20x20));
			Set( "BTEditor.Graph.NewTask", new IMAGE_BRUSH_SVG( "Starship/Common/Tasks", Icon20x20));
			Set( "BTEditor.Graph.NewDecorator", new IMAGE_BRUSH_SVG( "Starship/BehaviorTree/BlackboardDecorator", Icon20x20));
			Set( "BTEditor.Graph.NewService", new IMAGE_BRUSH_SVG( "Starship/BehaviorTree/BlackboardService", Icon20x20));

			Set( "BTEditor.Blackboard", new IMAGE_BRUSH_SVG( "Starship/BehaviorTree/Blackboard", Icon16x16));
			Set( "BTEditor.BehaviorTree", new IMAGE_BRUSH_SVG( "Starship/AssetIcons/BehaviorTree_16", Icon16x16));
		}
		
		{
			Set("EnvQueryEditor.Profiler.LoadStats", new IMAGE_BRUSH("Icons/LV_Load", Icon40x40));
			Set("EnvQueryEditor.Profiler.SaveStats", new IMAGE_BRUSH("Icons/LV_Save", Icon40x40));
		}

		// Visible on hover button for transition node
		{
			Set( "TransitionNodeButton.Normal", new FSlateNoResource() );
			Set( "TransitionNodeButton.Hovered", new IMAGE_BRUSH( "/Persona/StateMachineEditor/Trans_Button_Hovered", FVector2f(12.f,25.f) ) );
			Set( "TransitionNodeButton.Pressed", new IMAGE_BRUSH( "/Persona/StateMachineEditor/Trans_Button_Pressed", FVector2f(12.f,25.f) ) );
		}

		{
			Set( "Graph.AnimationResultNode.Body", new IMAGE_BRUSH( "/Graph/Animation/AnimationNode_Result_128x", Icon128x128 ) );
			Set( "Graph.AnimationFastPathIndicator", new IMAGE_BRUSH( "/Graph/Animation/AnimationNode_FastPath", Icon32x32 ) );
		}

		// SoundCueEditor Graph Nodes
		{
			Set( "Graph.SoundResultNode.Body", new IMAGE_BRUSH( "/Graph/SoundCue_SpeakerIcon", FVector2f(144.f, 144.f) ) );
		}

		Set( "Graph.Node.NodeEntryTop", new IMAGE_BRUSH( "Old/Graph/NodeEntryTop", FVector2f(64.f,12.f) ) );
		Set( "Graph.Node.NodeEntryBottom", new IMAGE_BRUSH( "Old/Graph/NodeEntryBottom", FVector2f(64.f,12.f) ) );
		Set( "Graph.Node.NodeExitTop", new IMAGE_BRUSH( "Old/Graph/NodeExitTop", FVector2f(64.f,12.f) ) );
		Set( "Graph.Node.NodeExitBottom", new IMAGE_BRUSH( "Old/Graph/NodeExitBottom", FVector2f(64.f,12.f) ) );

		Set( "Graph.Node.NodeEntryShadow", new BOX_BRUSH( "Old/Graph/NodeEntryShadow", FMargin(5.f/80, 21.f/52) ) );
		Set( "Graph.Node.NodeEntryShadowSelected", new BOX_BRUSH( "Old/Graph/NodeEntryShadowSelected", FMargin(5.f/80, 21.f/52) ) );
		Set( "Graph.Node.NodeExitShadow", new BOX_BRUSH( "Old/Graph/NodeExitShadow", FMargin(5.f/80, 21.f/52) ) );
		Set( "Graph.Node.NodeExitShadowSelected", new BOX_BRUSH( "Old/Graph/NodeExitShadowSelected", FMargin(5.f/80, 21.f/52) ) );

		Set( "Graph.Node.Autoplay", new IMAGE_BRUSH( "Graph/Icons/Overlay_Autoplay", FVector2f(22.f,22.f) ) );
		Set( "Graph.Node.Loop", new IMAGE_BRUSH( "Graph/Icons/Overlay_Loop", FVector2f(22.f,22.f) ) );

		{
			FTextBlockStyle GraphNodeTitle = FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Bold", 10 ) )
				.SetColorAndOpacity( FStyleColors::Foreground )
				.SetShadowOffset( FVector2f::ZeroVector )
				.SetShadowColorAndOpacity( FLinearColor(0.f,0.f,0.f, 0.7f) );
			Set( "Graph.Node.NodeTitle", GraphNodeTitle );

			FEditableTextBoxStyle GraphNodeTitleEditableText = FEditableTextBoxStyle(NormalEditableTextBoxStyle)
				.SetFont(NormalText.Font)
				.SetForegroundColor(FStyleColors::Input)
				.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Foreground, FStyleColors::Secondary, 1.0f))
				.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Foreground, FStyleColors::Hover, 1.0f))
				.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Foreground, FStyleColors::Primary, 1.0f))
				.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, FStyleColors::InputOutline, 1.0f))
				.SetForegroundColor(FStyleColors::Background)
				.SetBackgroundColor(FStyleColors::White)
				.SetReadOnlyForegroundColor(FStyleColors::Foreground)
				.SetFocusedForegroundColor(FStyleColors::Background)
				.SetScrollBarStyle( ScrollBar );
			Set( "Graph.Node.NodeTitleEditableText", GraphNodeTitleEditableText );

			Set( "Graph.Node.NodeTitleInlineEditableText", FInlineEditableTextBlockStyle()
				.SetTextStyle(GraphNodeTitle)
				.SetEditableTextBoxStyle(GraphNodeTitleEditableText)
			);

			Set( "Graph.Node.NodeTitleExtraLines", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Italic", 9 ) )
				.SetColorAndOpacity( FLinearColor(218.0f/255.0f,218.0f/255.0f,96.0f/255.0f, 0.5f) )
				.SetShadowOffset( FVector2f::ZeroVector )
				.SetShadowColorAndOpacity( FLinearColor(0.f,0.f,0.f, 0.7f) )
			);

			FEditableTextBoxStyle CommentEditableText = FEditableTextBoxStyle(NormalEditableTextBoxStyle)
				.SetFont(NormalText.Font)
				.SetForegroundColor(FStyleColors::Foreground)
				.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Foreground, CoreStyleConstants::InputFocusRadius, FStyleColors::Secondary, CoreStyleConstants::InputFocusThickness))
				.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Foreground, CoreStyleConstants::InputFocusRadius, FStyleColors::Hover, CoreStyleConstants::InputFocusThickness))
				.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Foreground, CoreStyleConstants::InputFocusRadius, FStyleColors::Primary, CoreStyleConstants::InputFocusThickness))
				.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
				.SetForegroundColor(FStyleColors::Background)
				.SetBackgroundColor(FStyleColors::White)
				.SetReadOnlyForegroundColor(FSlateColor::UseForeground())
				.SetFocusedForegroundColor(FStyleColors::White)
				.SetScrollBarStyle(ScrollBar);
			Set("Graph.CommentBubble.EditableText", CommentEditableText);
		
			FTextBlockStyle GraphCommentBlockTitle = FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Bold", 18 ) )
				.SetColorAndOpacity( FLinearColor(218.0f/255.0f,218.0f/255.0f,218.0f/255.0f) )
				.SetShadowOffset( FVector2f(1.5f, 1.5f) )
				.SetShadowColorAndOpacity( FLinearColor(0.f,0.f,0.f, 0.7f) );
			Set( "Graph.CommentBlock.Title", GraphCommentBlockTitle );

			FEditableTextBoxStyle GraphCommentBlockTitleEditableText = FEditableTextBoxStyle(GraphNodeTitleEditableText)
				.SetFont(GraphCommentBlockTitle.Font)
				.SetScrollBarStyle( ScrollBar )
				.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Foreground, 0.0f, FStyleColors::Transparent, 0.0f))
				.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Foreground, CoreStyleConstants::InputFocusRadius, FStyleColors::Hover, CoreStyleConstants::InputFocusThickness))
				.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Foreground, CoreStyleConstants::InputFocusRadius, FStyleColors::Primary, CoreStyleConstants::InputFocusThickness))
				.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness));
			Set( "Graph.CommentBlock.TitleEditableText", GraphCommentBlockTitleEditableText );

			Set( "Graph.CommentBlock.TitleInlineEditableText", FInlineEditableTextBlockStyle()
				.SetTextStyle(GraphCommentBlockTitle)
				.SetEditableTextBoxStyle(GraphCommentBlockTitleEditableText)
				);

			Set( "Graph.CompactNode.Title", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "BoldCondensed", 20 ) )
				.SetColorAndOpacity( FLinearColor(1.0f, 1.0f, 1.0f, 0.5f) )
				.SetShadowOffset( FVector2f::ZeroVector )
				.SetShadowColorAndOpacity( FLinearColor::White )
			);

			Set( "Graph.ArrayCompactNode.Title", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "BoldCondensed", 20 ) )
				.SetColorAndOpacity( FLinearColor(1.0f, 1.0f, 1.0f, 0.5f) ) //218.0f/255.0f, 218.0f/255.0f, 218.0f/255.0f, 0.25f) )
				.SetShadowOffset( FVector2f::ZeroVector )
				.SetShadowColorAndOpacity( FLinearColor::White )
				);

			Set( "Graph.Node.PinName", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Regular", 9 ) )
				.SetColorAndOpacity( FLinearColor(218.0f/255.0f,218.0f/255.0f,218.0f/255.0f) )
				.SetShadowOffset( FVector2f::ZeroVector )
				.SetShadowColorAndOpacity( FLinearColor(0.8f,0.8f,0.8f, 0.5) )
			);

			// Inline Editable Text Block
			{
				FTextBlockStyle InlineEditableTextBlockReadOnly = FTextBlockStyle(NormalText)
					.SetFont(DEFAULT_FONT("Regular", 9))
					.SetColorAndOpacity(FLinearColor(218.0f / 255.0f, 218.0f / 255.0f, 218.0f / 255.0f))
					.SetShadowOffset(FVector2f::ZeroVector)
					.SetShadowColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f, 0.5));

				FEditableTextBoxStyle InlineEditableTextBlockEditable = FEditableTextBoxStyle()
					.SetTextStyle(NormalText)
					.SetFont(DEFAULT_FONT("Regular", 9))
					.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
					.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Hover, CoreStyleConstants::InputFocusThickness))
					.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Primary, CoreStyleConstants::InputFocusThickness))
					.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, CoreStyleConstants::InputFocusRadius, FStyleColors::InputOutline, CoreStyleConstants::InputFocusThickness))
					.SetScrollBarStyle(ScrollBar);

				FInlineEditableTextBlockStyle InlineEditableTextBlockStyle = FInlineEditableTextBlockStyle()
					.SetTextStyle(InlineEditableTextBlockReadOnly)
					.SetEditableTextBoxStyle(InlineEditableTextBlockEditable);

				Set("Graph.Node.InlineEditablePinName", InlineEditableTextBlockStyle);
			}
		}

		{
			const FLinearColor BrighterColor(1.0f, 1.0f, 1.0f, 0.4f);
			const FLinearColor DarkerColor(0.8f, 0.8f, 0.8f, 0.4f);
			const float MarginSize = 9.0f/16.0f;

			/* Set states for various SCheckBox images ... */
			const FCheckBoxStyle GraphNodeAdvancedViewCheckBoxStyle = FCheckBoxStyle()
				.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
				.SetUncheckedImage( FSlateNoResource() )
				.SetUncheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x", MarginSize, DarkerColor ) )
				.SetCheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x", MarginSize, BrighterColor ) )
				.SetCheckedImage( FSlateNoResource() )
				.SetCheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x", MarginSize, DarkerColor ) )
				.SetCheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x", MarginSize, BrighterColor ) );
			/* ... and add new style */
			Set( "Graph.Node.AdvancedView", GraphNodeAdvancedViewCheckBoxStyle );
		}

		// Special style for switch statements default pin label
		{
			Set( "Graph.Node.DefaultPinName", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "Italic", 9 ) )
				.SetColorAndOpacity( FLinearColor(218.0f/255.0f,218.0f/255.0f,218.0f/255.0f) )
				.SetShadowOffset( FVector2f::ZeroVector )
				.SetShadowColorAndOpacity( FLinearColor(0.8f,0.8f,0.8f, 0.5) )
			);
		}
		Set( "Graph.Pin.DefaultPinSeparator", new IMAGE_BRUSH( "/Graph/PinSeparator", FVector2f(64.f,8.f) ) );

		/** Original Pin Styles */
		Set( "Graph.Pin.Connected", new IMAGE_BRUSH( "/Graph/Pin_connected", FVector2f(11.f,11.f) ) );
		Set( "Graph.Pin.Disconnected", new IMAGE_BRUSH( "/Graph/Pin_disconnected", FVector2f(11.f,11.f) ) );
		Set( "Graph.ArrayPin.Connected", new IMAGE_BRUSH( "/Graph/ArrayPin_connected", FVector2f(11.f,11.f) ) );
		Set( "Graph.ArrayPin.Disconnected", new IMAGE_BRUSH( "/Graph/ArrayPin_disconnected", FVector2f(11.f,11.f) ) );
		Set( "Graph.RefPin.Connected", new IMAGE_BRUSH( "/Graph/RefPin_connected", FVector2f(11.f,11.f) ) );
		Set( "Graph.RefPin.Disconnected", new IMAGE_BRUSH( "/Graph/RefPin_disconnected", FVector2f(11.f,11.f) ) );

		Set("Graph.Pin.CopyNodePinLeft_Connected", new IMAGE_BRUSH("/Graph/CopyNodePinLeft_connected", FVector2f(12.f, 24.f)));
		Set("Graph.Pin.CopyNodePinLeft_Disconnected", new IMAGE_BRUSH("/Graph/CopyNodePinLeft_disconnected", FVector2f(12.f, 24.f)));

		Set("Graph.Pin.CopyNodePinRight_Connected", new IMAGE_BRUSH("/Graph/CopyNodePinRight_connected", FVector2f(12.f, 24.f)));
		Set("Graph.Pin.CopyNodePinRight_Disconnected", new IMAGE_BRUSH("/Graph/CopyNodePinRight_disconnected", FVector2f(12.f, 24.f)));

		/** Variant A Pin Styles */
		Set( "Graph.Pin.Connected_VarA", new IMAGE_BRUSH( "/Graph/Pin_connected_VarA", FVector2f(15.f,11.f)) );
		Set( "Graph.Pin.Disconnected_VarA", new IMAGE_BRUSH( "/Graph/Pin_disconnected_VarA", FVector2f(15.f,11.f)) );

		Set( "Graph.DelegatePin.Connected", new IMAGE_BRUSH( "/Graph/DelegatePin_Connected", FVector2f(11.f,11.f) ) );
		Set( "Graph.DelegatePin.Disconnected", new IMAGE_BRUSH( "/Graph/DelegatePin_Disconnected", FVector2f(11.f,11.f) ) );

		Set( "Graph.Replication.AuthorityOnly", new IMAGE_BRUSH( "/Graph/AuthorityOnly", Icon32x32 ) );
		Set("Graph.Replication.ClientEvent", new IMAGE_BRUSH("/Graph/ClientEvent", Icon32x32));
		Set( "Graph.Replication.Replicated", new IMAGE_BRUSH( "/Graph/Replicated", Icon32x32 ) );

		Set("Graph.Editor.EditorOnlyIcon", new IMAGE_BRUSH("/Graph/EditorOnly", Icon32x32));

		Set( "Graph.Event.InterfaceEventIcon", new IMAGE_BRUSH("/Graph/InterfaceEventIcon", Icon32x32 ) );

		Set( "Graph.Latent.LatentIcon", new IMAGE_BRUSH("/Graph/LatentIcon", Icon32x32 ) );
		Set( "Graph.Message.MessageIcon", new IMAGE_BRUSH("/Graph/MessageIcon", Icon32x32 ) );
		Set( "Graph.Function.FunctionParameterIcon", new IMAGE_BRUSH_SVG("/Starship/GraphEditors/FunctionInputParameter", FVector2f(20.0f, 20.0f) ) );
		Set( "Graph.Function.FunctionLocalVariableIcon", new IMAGE_BRUSH_SVG("/Starship/GraphEditors/FunctionLocalVariable", FVector2f(20.0f, 20.0f) ) );

		Set( "Graph.ExecPin.Connected", new IMAGE_BRUSH( "Old/Graph/ExecPin_Connected", Icon12x16 ) );
		Set( "Graph.ExecPin.Disconnected", new IMAGE_BRUSH( "Old/Graph/ExecPin_Disconnected", Icon12x16 ) );
		Set( "Graph.ExecPin.ConnectedHovered", new IMAGE_BRUSH( "Old/Graph/ExecPin_Connected", Icon12x16, FLinearColor(0.8f,0.8f,0.8f) ) );
		Set( "Graph.ExecPin.DisconnectedHovered", new IMAGE_BRUSH( "Old/Graph/ExecPin_Disconnected", Icon12x16, FLinearColor(0.8f,0.8f,0.8f) ) );

		const FVector2f Icon15x28(15.0f, 28.0f);
		Set("Graph.PosePin.Connected", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphPose", Icon16x16));
		Set("Graph.PosePin.Disconnected", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphPoseDisconnected", Icon16x16));
		Set("Graph.PosePin.ConnectedHovered", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphPose", Icon16x16, FLinearColor(0.8f, 0.8f, 0.8f)));
		Set("Graph.PosePin.DisconnectedHovered", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphPoseDisconnected", Icon16x16, FLinearColor(0.8f, 0.8f, 0.8f)));

		// Events Exec Pins
		Set( "Graph.ExecEventPin.Connected", new IMAGE_BRUSH( "Graph/EventPin_Connected", Icon16x16 ) );
		Set( "Graph.ExecEventPin.Disconnected", new IMAGE_BRUSH( "Graph/EventPin_Disconnected", Icon16x16 ) );
		Set( "Graph.ExecEventPin.ConnectedHovered", new IMAGE_BRUSH( "Graph/EventPin_Connected", Icon16x16, FLinearColor(0.8f,0.8f,0.8f) ) );
		Set( "Graph.ExecEventPin.DisconnectedHovered", new IMAGE_BRUSH( "Graph/EventPin_Disconnected", Icon16x16, FLinearColor(0.8f,0.8f,0.8f) ) );

		Set( "Graph.WatchedPinIcon_Pinned", new IMAGE_BRUSH( "Old/Graph/WatchedPinIcon_Pinned", Icon16x16 ) );

		Set( "Graph.Pin.BackgroundHovered", new IMAGE_BRUSH( "/Graph/Pin_hover_cue", FVector2f(32.f,8.f)));
		Set( "Graph.Pin.Background", new FSlateNoResource() );
		Set( "Graph.Pin.DiffHighlight", new IMAGE_BRUSH( "/Graph/Pin_DiffHighlight", FVector2f(32.f,8.f)));

		Set( "Graph.Pin.ObjectSet", new IMAGE_BRUSH( "Old/Graph/Pin_ObjectSet", Icon12x12 ) );
		Set( "Graph.Pin.ObjectEmpty", new IMAGE_BRUSH( "Old/Graph/Pin_ObjectEmpty", Icon12x12 ) );

		Set("Graph.Pin.Dummy", new IMAGE_BRUSH("/Graph/Pin_dummy", FVector2f(15.f, 11.f)));

		Set( "Graph.ConnectorFeedback.Border", new BOX_BRUSH( "Old/Menu_Background", FMargin(8.0f/64.0f) ) );
		Set( "Graph.ConnectorFeedback.OK", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check-circle", Icon16x16, FStyleColors::AccentGreen));
		Set( "Graph.ConnectorFeedback.OKWarn", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check-circle", Icon16x16, FStyleColors::AccentYellow));
		Set( "Graph.ConnectorFeedback.Error", new IMAGE_BRUSH( "Old/Graph/Feedback_Error", Icon16x16 ) );
		Set( "Graph.ConnectorFeedback.NewNode", new IMAGE_BRUSH( "Old/Graph/Feedback_NewNode", Icon16x16 ) );
		Set( "Graph.ConnectorFeedback.ViaCast", new IMAGE_BRUSH( "Old/Graph/Feedback_ConnectViaCast", Icon16x16 ) );
		Set( "Graph.ConnectorFeedback.ShowNode", new IMAGE_BRUSH( "Graph/Feedback_ShowNode", Icon16x16 ) );

		{
			Set( "Graph.CornerText", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "BoldCondensed", 48 ) )
				.SetColorAndOpacity( FLinearColor(0.8f, 0.8f, 0.8f, 0.2f) )
				.SetShadowOffset( FVector2f::ZeroVector )
			);

			Set("Graph.WarningText", FTextBlockStyle(NormalText)
				.SetFont(DEFAULT_FONT("BoldCondensed", 20))
				.SetColorAndOpacity(FLinearColor(0.8f, 0.5f, 0.07f, 1.0f))
				.SetShadowOffset(FVector2f::UnitVector)
			);

			Set( "Graph.SimulatingText", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "BoldCondensed", 48 ) )
				.SetColorAndOpacity( FLinearColor(0.8f, 0.8f, 0.0f, 0.2f) )
				.SetShadowOffset( FVector2f::ZeroVector )
			);

			Set( "GraphPreview.CornerText", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "BoldCondensed", 16 ) )
				.SetColorAndOpacity( FLinearColor(0.8f, 0.8f, 0.8f, 0.2f) )
				.SetShadowOffset( FVector2f::ZeroVector )
			);

			Set( "Graph.InstructionText", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "BoldCondensed", 24 ) )
				.SetColorAndOpacity( FLinearColor(1.f, 1.f, 1.f, 0.6f) )
				.SetShadowOffset( FVector2f::ZeroVector )
			);

			Set( "Graph.InstructionBackground", new BOX_BRUSH("Common/GroupBorder", FMargin(4.0f / 16.0f), FLinearColor(0.1f, 0.1f, 0.1f, 0.7f)) );
		}

		{
			Set( "Graph.ZoomText", FTextBlockStyle(NormalText)
				.SetFont( DEFAULT_FONT( "BoldCondensed", 16 ) )
			);
		}

		Set("ClassIcon.K2Node_CallFunction",	new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_Function", Icon16x16));
		Set("ClassIcon.K2Node_FunctionEntry",	new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_Function", Icon16x16));
		Set("ClassIcon.K2Node_CustomEvent",		new IMAGE_BRUSH_SVG("Starship/Common/Event", Icon16x16));
		Set("ClassIcon.K2Node_Event",			new IMAGE_BRUSH_SVG("Starship/Common/Event", Icon16x16));
		Set("ClassIcon.K2Node_Variable",		new IMAGE_BRUSH_SVG("Starship/GraphEditors/Node", Icon16x16, FLinearColor::White));
		Set("ClassIcon.K2Node_VariableGet",		new IMAGE_BRUSH_SVG("Starship/GraphEditors/VarGet", Icon16x16, FLinearColor::White));
		Set("ClassIcon.K2Node_VariableSet",		new IMAGE_BRUSH_SVG("Starship/GraphEditors/VarSet", Icon16x16, FLinearColor::White));
		Set("ClassIcon.K2Node_DynamicCast",		new IMAGE_BRUSH_SVG("Starship/GraphEditors/Cast", Icon16x16));

		Set("GraphEditor.Clean",				new IMAGE_BRUSH_SVG("Starship/GraphEditors/CleanUp", Icon20x20));
		Set("GraphEditor.OverrideFunction_16x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_OverrideFunction", Icon16x16));
		Set("GraphEditor.OverrideFunction_24x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_OverrideFunction", Icon24x24));
		Set("GraphEditor.EventGraph_16x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_BlueprintEditor_EventGraph", Icon16x16));
		Set("GraphEditor.EventGraph_24x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_BlueprintEditor_EventGraph", Icon24x24));
		Set("GraphEditor.Macro_16x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_Macro", Icon16x16));
		Set("GraphEditor.Macro_24x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_Macro", Icon24x24));
		Set("GraphEditor.Function_16x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_Function", Icon16x16));
		Set("GraphEditor.Function_24x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_Function", Icon24x24));
		Set("GraphEditor.Delegate_16x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_Delegate", Icon16x16));
		Set("GraphEditor.Delegate_24x", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_Delegate", Icon24x24));



		Set( "GraphEditor.Default_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/Node", Icon16x16));
		Set( "GraphEditor.InterfaceFunction_16x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/InterfaceFunction", Icon16x16));
		Set( "GraphEditor.PureFunction_16x", new IMAGE_BRUSH_SVG( "Starship/Blueprints/icon_Blueprint_Function", Icon16x16 ) );
		Set( "GraphEditor.PotentialOverrideFunction_16x", new IMAGE_BRUSH_SVG( "Starship/Blueprints/icon_Blueprint_OverrideFunction", Icon16x16 ) );
		Set( "GraphEditor.OverridePureFunction_16x", new IMAGE_BRUSH_SVG( "Starship/Blueprints/icon_Blueprint_OverrideFunction", Icon16x16 ) );
		Set( "GraphEditor.SubGraph_16x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/SubGraph", Icon16x16 ) );
		Set( "GraphEditor.Animation_16x", new IMAGE_BRUSH_SVG( "Starship/Common/Animation", Icon16x16 ) );
		Set( "GraphEditor.Conduit_16x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/Conduit", Icon16x16 ) );
		Set( "GraphEditor.Rule_16x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/Rule", Icon16x16 ) );
		Set( "GraphEditor.State_16x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/State", Icon16x16 ) );
		Set( "GraphEditor.StateMachine_16x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/StateMachine", Icon16x16 ) );
		Set( "GraphEditor.Event_16x", new IMAGE_BRUSH_SVG( "Starship/Common/Event", Icon16x16 ) );
		Set( "GraphEditor.CustomEvent_16x", new IMAGE_BRUSH_SVG( "Starship/Common/Event", Icon16x16 ) );
		Set( "GraphEditor.CallInEditorEvent_16x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/CallInEditorEvent", Icon16x16 ) );
		Set( "GraphEditor.Timeline_16x", new IMAGE_BRUSH_SVG("Starship/Common/Timecode", Icon16x16));
		Set( "GraphEditor.Documentation_16x", new IMAGE_BRUSH_SVG("Starship/Common/Documentation", Icon16x16));
		Set( "GraphEditor.Switch_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/Switch", Icon16x16));
		Set( "GraphEditor.BreakStruct_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/BreakStruct", Icon16x16));
		Set( "GraphEditor.MakeStruct_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/MakeStruct", Icon16x16));
		Set( "GraphEditor.Sequence_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/Sequence", Icon16x16));
		Set( "GraphEditor.Branch_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/Branch", Icon16x16));
		Set( "GraphEditor.SpawnActor_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/SpawnActor", Icon16x16));
		Set( "GraphEditor.PadEvent_16x", new CORE_IMAGE_BRUSH_SVG("Starship/Common/PlayerController", Icon16x16));
		Set( "GraphEditor.MouseEvent_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/MouseEvent", Icon16x16));
		Set( "GraphEditor.KeyEvent_16x", new IMAGE_BRUSH_SVG("Starship/Common/ViewportControls", Icon16x16));
		Set( "GraphEditor.TouchEvent_16x", new IMAGE_BRUSH_SVG("Starship/Common/TouchInterface", Icon16x16));
		Set( "GraphEditor.MakeArray_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/MakeArray", Icon16x16));
		Set( "GraphEditor.MakeSet_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/MakeSet", Icon16x16));
		Set( "GraphEditor.MakeMap_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/MakeMap", Icon16x16));
		Set( "GraphEditor.Enum_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/Enum", Icon16x16));
		Set( "GraphEditor.Select_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/Select", Icon16x16));
		Set( "GraphEditor.Cast_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/Cast", Icon16x16));

		Set( "GraphEditor.Macro.Loop_16x", new IMAGE_BRUSH_SVG("Starship/Common/Loop", Icon16x16));
		Set( "GraphEditor.Macro.Gate_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/Gate", Icon16x16));
		Set( "GraphEditor.Macro.DoN_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/DoN", Icon16x16));
		Set( "GraphEditor.Macro.DoOnce_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/DoOnce", Icon16x16));
		Set( "GraphEditor.Macro.IsValid_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/IsValid", Icon16x16));
		Set( "GraphEditor.Macro.FlipFlop_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/FlipFlop", Icon16x16));
		Set( "GraphEditor.Macro.ForEach_16x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/ForEach", Icon16x16));

		// The 24x icons are used for the SGraphTitleBar that shows the breadcrumbs for subgraphs
		Set( "GraphEditor.SubGraph_24x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/SubGraph", Icon24x24 ) );
		Set( "GraphEditor.Animation_24x", new IMAGE_BRUSH_SVG( "Starship/Common/Animation", Icon24x24 ) );
		Set( "GraphEditor.Conduit_24x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/Conduit", Icon24x24 ) );
		Set( "GraphEditor.Rule_24x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/Rule", Icon24x24 ) );
		Set( "GraphEditor.State_24x", new IMAGE_BRUSH_SVG("Starship/GraphEditors/State", Icon24x24));
		Set( "GraphEditor.StateMachine_24x", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/StateMachine", Icon24x24 ) );

		Set( "GraphEditor.NodeGlyph", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/Node", Icon16x16, FLinearColor::White) );
		Set( "GraphEditor.PinIcon", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/PinIcon", Icon16x16, FLinearColor::White) );
		Set( "GraphEditor.ArrayPinIcon", new IMAGE_BRUSH_SVG( "Starship/Blueprints/pillarray", Icon16x16, FLinearColor::White ) );
		Set( "GraphEditor.RefPinIcon", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/RefPin", Icon16x16, FLinearColor::White ) );
		Set( "GraphEditor.EnumGlyph", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/Enum", Icon16x16, FLinearColor::White) );
		Set( "GraphEditor.TimelineGlyph", new IMAGE_BRUSH_SVG( "Starship/Common/Timecode", Icon16x16, FLinearColor::White) );
		Set( "GraphEditor.StructGlyph", new IMAGE_BRUSH_SVG( "Starship/Common/Struct", Icon16x16, FLinearColor::White) );

		// Find In Blueprints
		Set( "GraphEditor.FIB_Event", new IMAGE_BRUSH_SVG( "Starship/Common/Event", Icon16x16, FLinearColor::White) );

		Set( "GraphEditor.GetSequenceBinding", new IMAGE_BRUSH_SVG("Starship/GraphEditors/GetSequenceBinding", Icon16x16));

		Set( "GraphEditor.GoToDocumentation", new IMAGE_BRUSH_SVG( "Starship/Common/Documentation", Icon16x16 ) );

		Set( "GraphEditor.AlignNodesTop", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/AlignTop", Icon20x20 ) );
		Set( "GraphEditor.AlignNodesMiddle", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/AlignMiddle", Icon20x20 ) );
		Set( "GraphEditor.AlignNodesBottom", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/AlignBottom", Icon20x20 ) );
		Set( "GraphEditor.AlignNodesLeft", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/AlignLeft", Icon20x20 ) );
		Set( "GraphEditor.AlignNodesCenter", new IMAGE_BRUSH_SVG( "Starship/Common/Align", Icon20x20 ) );
		Set( "GraphEditor.AlignNodesRight", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/AlignRight", Icon20x20 ) );

		Set( "GraphEditor.StraightenConnections", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/StraightenConnections", Icon20x20 ) );

		Set( "GraphEditor.DistributeNodesHorizontally", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/DistributeHorizontally", Icon20x20 ) );
		Set( "GraphEditor.DistributeNodesVertically", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/DistributeVertically", Icon20x20 ) );

		Set( "GraphEditor.StackNodesHorizontally", new IMAGE_BRUSH_SVG("Starship/GraphEditors/StackHorizontally", Icon20x20) );
		Set( "GraphEditor.StackNodesVertically", new IMAGE_BRUSH_SVG("Starship/GraphEditors/StackVertically", Icon20x20) );
		
		Set( "GraphEditor.ToggleHideUnrelatedNodes", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/HideUnrelated", Icon20x20 ) );
		Set( "GraphEditor.Bookmark", new IMAGE_BRUSH_SVG("Starship/Common/Bookmarks", Icon16x16) );
		Set( "GraphEditor.Bookmark.Outline", new IMAGE_BRUSH_SVG("Starship/Common/BookmarksOutline", Icon16x16));

		// Graph editor widgets
		{
			// EditableTextBox
			{
				Set( "Graph.EditableTextBox", FEditableTextBoxStyle()
					.SetTextStyle(NormalText)
					.SetBackgroundImageNormal( BOX_BRUSH( "Graph/CommonWidgets/TextBox", FMargin(4.0f/16.0f) ) )
					.SetBackgroundImageHovered( BOX_BRUSH( "Graph/CommonWidgets/TextBox_Hovered", FMargin(4.0f/16.0f) ) )
					.SetBackgroundImageFocused( BOX_BRUSH( "Graph/CommonWidgets/TextBox_Hovered", FMargin(4.0f/16.0f) ) )
					.SetBackgroundImageReadOnly( BOX_BRUSH( "Graph/CommonWidgets/TextBox", FMargin(4.0f/16.0f) ) )
					.SetScrollBarStyle( ScrollBar )
					);
			}

			// VectorEditableTextBox
			{
				Set( "Graph.VectorEditableTextBox", FEditableTextBoxStyle()
					.SetTextStyle(NormalText)
					.SetBackgroundImageNormal( BOX_BRUSH( "Graph/CommonWidgets/TextBox", FMargin(4.0f/16.0f) ) )
					.SetBackgroundImageHovered( BOX_BRUSH( "Graph/CommonWidgets/TextBox_Hovered", FMargin(4.0f/16.0f) ) )
					.SetBackgroundImageFocused( BOX_BRUSH( "Graph/CommonWidgets/TextBox_Hovered", FMargin(4.0f/16.0f) ) )
					.SetBackgroundImageReadOnly( BOX_BRUSH( "Graph/CommonWidgets/TextBox", FMargin(4.0f/16.0f) ) )
					.SetScrollBarStyle( ScrollBar )
					.SetForegroundColor( FLinearColor::White )
					.SetBackgroundColor( FLinearColor::Blue )
					);
			}

			// Check Box
			{
				/* Set images for various SCheckBox states of style Graph.Checkbox ... */
				const FCheckBoxStyle BasicGraphCheckBoxStyle = FCheckBoxStyle()
					.SetUncheckedImage( IMAGE_BRUSH( "/Graph/CommonWidgets/CheckBox", Icon20x20 ) )
					.SetUncheckedHoveredImage( IMAGE_BRUSH( "/Graph/CommonWidgets/CheckBox_Hovered", Icon20x20 ) )
					.SetUncheckedPressedImage( IMAGE_BRUSH( "/Graph/CommonWidgets/CheckBox_Hovered", Icon20x20 ) )
					.SetCheckedImage( IMAGE_BRUSH( "/Graph/CommonWidgets/CheckBox_Checked", Icon20x20 ) )
					.SetCheckedHoveredImage( IMAGE_BRUSH( "/Graph/CommonWidgets/CheckBox_Checked_Hovered", Icon20x20 ) )
					.SetCheckedPressedImage( IMAGE_BRUSH( "/Graph/CommonWidgets/CheckBox_Checked", Icon20x20, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
					.SetUndeterminedImage( IMAGE_BRUSH( "/Graph/CommonWidgets/CheckBox_Undetermined", Icon20x20 ) )
					.SetUndeterminedHoveredImage( IMAGE_BRUSH( "/Graph/CommonWidgets/CheckBox_Undetermined_Hovered", Icon20x20 ) )
					.SetUndeterminedPressedImage( IMAGE_BRUSH( "/Graph/CommonWidgets/CheckBox_Undetermined_Hovered", Icon20x20, FLinearColor( 0.5f, 0.5f, 0.5f ) ) );

				/* ... and add the new style */
				Set( "Graph.Checkbox", BasicGraphCheckBoxStyle );
			}
		}

		// Timeline Editor
		{
			Set( "TimelineEditor.AddFloatTrack", new IMAGE_BRUSH_SVG( "Starship/TimelineEditor/TrackTypeFloat", Icon16x16) );
			Set( "TimelineEditor.AddVectorTrack", new IMAGE_BRUSH_SVG( "Starship/TimelineEditor/TrackTypeVector", Icon16x16) );
			Set( "TimelineEditor.AddEventTrack", new IMAGE_BRUSH_SVG( "Starship/Common/Event", Icon16x16) );
			Set( "TimelineEditor.AddColorTrack", new IMAGE_BRUSH_SVG( "Starship/TimelineEditor/TrackTypeColor", Icon16x16) );
			Set( "TimelineEditor.AddCurveAssetTrack", new IMAGE_BRUSH_SVG("Starship/TimelineEditor/TrackTypeCurve", Icon16x16));
			Set( "TimelineEditor.DeleteTrack", new IMAGE_BRUSH( "Icons/icon_TrackDelete_36x24px", Icon36x24 ) );
			Set("TimelineEditor.AutoPlay", new IMAGE_BRUSH_SVG("Starship/TimelineEditor/TimelineAutoPlay", Icon20x20));
			Set("TimelineEditor.IgnoreTimeDilation", new IMAGE_BRUSH_SVG("Starship/TimelineEditor/TimelineIgnoreTimeDilation", Icon20x20));
			Set("TimelineEditor.Replicated", new IMAGE_BRUSH_SVG("Starship/TimelineEditor/TimelineReplicated", Icon20x20));
			Set("TimelineEditor.UseLastKeyframe", new IMAGE_BRUSH_SVG("Starship/TimelineEditor/TimelineUseLastKeyframe", Icon20x20));
			Set("TimelineEditor.Loop", new IMAGE_BRUSH_SVG("Starship/Common/Loop", Icon20x20));

			Set("TimelineEditor.TrackRowSubtleHighlight", FTableRowStyle(NormalTableRowStyle)
				.SetActiveBrush(FSlateColorBrush(FStyleColors::Panel))
				.SetActiveHoveredBrush(FSlateColorBrush(FStyleColors::Header))
				.SetInactiveBrush(FSlateColorBrush(FStyleColors::Recessed))
				.SetInactiveHoveredBrush(FSlateColorBrush(FStyleColors::Panel))
				.SetActiveHighlightedBrush(FSlateColorBrush(FStyleColors::Panel)) // This is the parent hightlight
				.SetInactiveHighlightedBrush(FSlateColorBrush(FStyleColors::Recessed))// This is the parent highlight
			);
		}
	}

		// SCSEditor
	{
		Set("SCSEditor.TileViewTooltip.NonContentBorder", new BOX_BRUSH("/Docking/TabContentArea", FMargin(4 / 16.0f)));

		Set("SCSEditor.PromoteToBlueprintIcon", new IMAGE_BRUSH_SVG("Starship/Common/blueprint", Icon16x16));

		Set("SCSEditor.TopBar.Font", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 10))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

		Set("SCSEditor.TreePanel", new FSlateNoResource());
		Set("SCSEditor.Background", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f));

		//

		Set("SCSEditor.ComponentTooltip.Title",
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 12))
			.SetColorAndOpacity(FLinearColor::Black)
			);

		Set("SCSEditor.ComponentTooltip.Label",
			FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FLinearColor(0.075f, 0.075f, 0.075f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f))
		);
		Set("SCSEditor.ComponentTooltip.ImportantLabel",
			FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FLinearColor(0.05f, 0.05f, 0.05f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f))
		);


		Set("SCSEditor.ComponentTooltip.Value", 
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 10))
			.SetColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f))
		);
		Set("SCSEditor.ComponentTooltip.ImportantValue",
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 10))
			.SetColorAndOpacity(FLinearColor(0.3f, 0.0f, 0.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f))
		);

		Set("SCSEditor.ComponentTooltip.ClassDescription",
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Italic", 10))
			.SetColorAndOpacity(FLinearColor(0.1f, 0.1f, 0.1f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.8f, 0.8f, 0.8f))
		);
	}

	// Notify editor
	{
		Set( "Persona.NotifyEditor.NotifyTrackBackground", new BOX_BRUSH( "/Persona/NotifyEditor/NotifyTrackBackground", FMargin(8.0f/64.0f, 3.0f/32.0f) ) );
	}

	// Blueprint modes
	{
		Set( "ModeSelector.ToggleButton.Normal", new FSlateNoResource() );		// Note: Intentionally transparent background
		Set( "ModeSelector.ToggleButton.Pressed", new BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) );
		Set( "ModeSelector.ToggleButton.Hovered", new BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ) );


		Set( "BlueprintEditor.PipelineSeparator", new BOX_BRUSH( "Old/Kismet2/BlueprintModeSeparator", FMargin(15.0f/16.0f, 20.0f/20.0f, 1.0f/16.0f, 0.0f/20.0f), FLinearColor(1,1,1,0.5f) ) );
	}

	// Persona modes
	{
		Set( "Persona.PipelineSeparator", new BOX_BRUSH( "Persona/Modes/PipelineSeparator", FMargin(15.0f/16.0f, 22.0f/24.0f, 1.0f/16.0f, 1.0f/24.0f), FLinearColor(1,1,1,0.5f) ) );
	}

	// montage editor
	{
		Set("Persona.MontageEditor.ChildMontageInstruction", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("BoldCondensed", 14))
			.SetColorAndOpacity(FLinearColor(1.f, 1.f, 1.f, 1.0f))
			.SetShadowOffset(FVector2f::ZeroVector)
		);
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	}

void FStarshipEditorStyle::FStyle::SetupLevelEditorStyle()
	{
	// Level editor tool bar icons
#if WITH_EDITOR
	{
		Set("LevelEditor.BrowseDocumentation",     new IMAGE_BRUSH_SVG("Starship/Common/Documentation", Icon16x16));
		Set("LevelEditor.Tutorials",               new IMAGE_BRUSH_SVG("Starship/Common/Tutorials", Icon16x16));
		Set("LevelEditor.BrowseViewportControls",  new IMAGE_BRUSH_SVG("Starship/Common/ViewportControls", Icon16x16));
		Set("LevelEditor.PasteHere",				new IMAGE_BRUSH_SVG("Starship/Actors/paste-here", Icon16x16));

		Set("LevelEditor.AllowArcballRotation",		    new IMAGE_BRUSH_SVG("Starship/Common/RotationArcball_16", Icon16x16));
		Set("LevelEditor.AllowScreenspaceRotation",		    new IMAGE_BRUSH_SVG("Starship/Common/RotationScreenspace_16", Icon16x16));
		Set("LevelEditor.EnableViewportHoverFeedback",		    new IMAGE_BRUSH_SVG("Starship/Common/PreselectionHighlight_16", Icon16x16));
		Set("LevelEditor.AllowGroupSelection",			new IMAGE_BRUSH_SVG("Starship/Common/GroupActors", Icon16x16));
		Set("LevelEditor.AllowTranslucentSelection",		new IMAGE_BRUSH_SVG("Starship/Common/Transparency", Icon16x16));
		Set("LevelEditor.EnableActorSnap",				new IMAGE_BRUSH_SVG("Starship/Common/SnapActor_16", Icon16x16));
		Set("LevelEditor.EnableVertexSnap",				new IMAGE_BRUSH_SVG("Starship/Common/SnapVertex_16", Icon16x16));
		Set("LevelEditor.InvertSelection",				new IMAGE_BRUSH_SVG("Starship/Common/SelectInvert_16", Icon16x16));
		Set("LevelEditor.PreserveNonUniformScale",		    new IMAGE_BRUSH_SVG("Starship/Common/ScaleNonUniform_16", Icon16x16));
		Set("LevelEditor.PreviewPlatform", new IMAGE_BRUSH_SVG("Starship/Common/PreviewPlatform_16", Icon16x16));

		Set("LevelEditor.SelectAllDescendants",			new IMAGE_BRUSH_SVG("Starship/Common/AllDescendants_16", Icon16x16));
		Set("LevelEditor.SelectImmediateChildren",		new IMAGE_BRUSH_SVG("Starship/Common/ImmediateChildren_16", Icon16x16));
		Set("LevelEditor.SelectNone",					new IMAGE_BRUSH_SVG("Starship/Common/DeselectAll_16", Icon16x16));
		Set("LevelEditor.ShowTransformWidget",			new IMAGE_BRUSH_SVG("Starship/Common/ShowTransformGizmo_16", Icon16x16));
		Set("LevelEditor.SnapCameraToObject", 			new IMAGE_BRUSH_SVG("Starship/Common/MoveCameraToObject_16", Icon16x16));
		Set("LevelEditor.SnapObjectToCamera", 			new IMAGE_BRUSH_SVG("Starship/Common/MoveObjectToCamera_16", Icon16x16));
		Set("LevelEditor.StrictBoxSelect",				new IMAGE_BRUSH_SVG("Starship/Common/StrictMarqueeSelection_16", Icon16x16));
		Set("LevelEditor.ToggleSocketSnapping", 			new IMAGE_BRUSH_SVG("Starship/Common/SnapSocket_16", Icon16x16));
		Set("LevelEditor.TransparentBoxSelect", 			new IMAGE_BRUSH_SVG("Starship/Common/MarqueeSelectOccluded_16", Icon16x16));
		Set("LevelEditor.ShowSelectionSubcomponents", 			new IMAGE_BRUSH_SVG("Starship/Common/ShowSubcomponents_16", Icon16x16));
		Set("LevelEditor.UseExperimentalGizmos", 			new IMAGE_BRUSH_SVG("Starship/Common/GizmoExperimental_16", Icon16x16));

		Set("LevelEditor.MaterialQuality", new IMAGE_BRUSH_SVG("Starship/Common/MaterialQuality_16", Icon16x16));
			
		
		Set("MainFrame.ToggleFullscreen",          new IMAGE_BRUSH_SVG("Starship/Common/EnableFullscreen", Icon16x16));
		Set("MainFrame.LoadLayout",                new IMAGE_BRUSH_SVG("Starship/Common/LayoutLoad", Icon16x16));
		Set("MainFrame.SaveLayout",                new IMAGE_BRUSH_SVG("Starship/Common/LayoutSave", Icon16x16));
		Set("MainFrame.RemoveLayout",              new IMAGE_BRUSH_SVG("Starship/Common/LayoutRemove", Icon16x16));

		Set("MainFrame.OpenIssueTracker",          new IMAGE_BRUSH_SVG("Starship/Common/IssueTracker", Icon16x16));
		Set("MainFrame.ReportABug",                new IMAGE_BRUSH_SVG("Starship/Common/Bug", Icon16x16));

		Set("SystemWideCommands.OpenDocumentation", new IMAGE_BRUSH_SVG("Starship/Common/Documentation", Icon16x16));
		Set("MainFrame.DocumentationHome",	        new IMAGE_BRUSH_SVG("Starship/Common/Documentation", Icon16x16));
		Set("MainFrame.BrowseAPIReference",         new IMAGE_BRUSH_SVG("Starship/Common/Documentation", Icon16x16));
		Set("MainFrame.BrowseCVars",                new CORE_IMAGE_BRUSH_SVG("Starship/Common/Console", Icon16x16));
		Set("MainFrame.VisitCommunityHome",         new IMAGE_BRUSH_SVG("Starship/Common/Community", Icon16x16));
		Set("MainFrame.VisitOnlineLearning",		new IMAGE_BRUSH_SVG("Starship/Common/Tutorials", Icon16x16));
		Set("MainFrame.VisitForums",                new IMAGE_BRUSH_SVG("Starship/Common/Forums", Icon16x16));
		Set("MainFrame.VisitSearchForAnswersPage",  new IMAGE_BRUSH_SVG("Starship/Common/QuestionAnswer", Icon16x16));
		Set("MainFrame.VisitCommunitySnippets",     new IMAGE_BRUSH_SVG("Starship/Common/FileLined", Icon16x16));
		Set("MainFrame.VisitSupportWebSite",        new IMAGE_BRUSH_SVG("Starship/Common/Support", Icon16x16));
		Set("MainFrame.VisitEpicGamesDotCom",       new IMAGE_BRUSH_SVG("About/EpicGamesLogo", Icon16x16));
		Set("MainFrame.AboutUnrealEd",              new IMAGE_BRUSH_SVG("About/UnrealLogo", Icon16x16));
		Set("MainFrame.CreditsUnrealEd",            new IMAGE_BRUSH_SVG("Starship/Common/Credits", Icon16x16));

		Set( "EditorViewport.SelectMode", new IMAGE_BRUSH_SVG("Starship/Common/TransformSelect_16", Icon16x16) );
		Set( "EditorViewport.TranslateMode", new IMAGE_BRUSH_SVG("Starship/Common/TransformMove_16", Icon16x16 ) );
		Set( "EditorViewport.RotateMode", new IMAGE_BRUSH_SVG("Starship/Common/TransformRotate_16", Icon16x16 ) );
		Set( "EditorViewport.ScaleMode", new IMAGE_BRUSH_SVG("Starship/Common/TransformScale_16", Icon16x16 ) );
		Set("EditorViewport.Speed", new IMAGE_BRUSH_SVG("Starship/EditorViewport/speed", Icon16x16));

		Set( "EditorViewport.TranslateRotateMode", new IMAGE_BRUSH_SVG("Starship/EditorViewport/TranslateRotate3D", Icon16x16 ) );
		Set( "EditorViewport.TranslateRotate2DMode", new IMAGE_BRUSH_SVG("Starship/EditorViewport/TranslateRotate2D", Icon16x16 ) );

		Set( "EditorViewport.ToggleRealTime", new IMAGE_BRUSH_SVG("Starship/Common/Realtime", Icon16x16));
		Set( "EditorViewport.ToggleRealTimeLocked", new IMAGE_BRUSH_SVG("Starship/Common/RealtimeOff_16", Icon16x16));
		Set( "EditorViewport.RealTimeReset", new IMAGE_BRUSH_SVG("Starship/Common/RealtimeReset_16", Icon16x16));

		Set( "EditorViewport.LocationGridSnap", new IMAGE_BRUSH_SVG("Starship/EditorViewport/grid", Icon16x16));
		Set( "EditorViewport.RotationGridSnap", new IMAGE_BRUSH_SVG("Starship/EditorViewport/angle", Icon16x16));

		Set( "EditorViewport.Layer2DSnap", new IMAGE_BRUSH("Starship/EditorViewport/Layer2DSnap", Icon14x14));

		Set("EditorViewport.ScaleGridSnap", new IMAGE_BRUSH_SVG( "Starship/EditorViewport/scale-grid-snap", Icon16x16 ) );
		Set("EditorViewport.ToggleSurfaceSnapping", new IMAGE_BRUSH_SVG( "Starship/EditorViewport/surface-snap", Icon16x16 ) );
		Set("EditorViewport.ToggleSurfaceSnapping", new IMAGE_BRUSH_SVG("Starship/EditorViewport/surface-snap", Icon16x16));
		Set("EditorViewport.ToggleInGameExposure", new IMAGE_BRUSH_SVG("Starship/Common/GameSettings_16", Icon16x16));

		Set("EditorViewport.RelativeCoordinateSystem_World", new IMAGE_BRUSH_SVG( "Starship/EditorViewport/globe", Icon16x16 ));
		Set("EditorViewport.RelativeCoordinateSystem_Local", new IMAGE_BRUSH_SVG("Starship/Common/transform-local", Icon16x16));
		Set("EditorViewport.RelativeCoordinateSystem_Parent", new CORE_IMAGE_BRUSH_SVG("Starship/Common/ParentHierarchy", Icon16x16));
		Set("EditorViewport.RelativeCoordinateSystem_Explicit", new IMAGE_BRUSH_SVG("Starship/Common/transform-explicit", Icon16x16));
		
		Set("EditorViewport.RestoreCoordinateSpaceOnSwitch",
			new IMAGE_BRUSH_SVG("Starship/EditorViewport/RestoreCoordinateSpace_16", Icon16x16));
		Set("EditorViewport.LocalTransformsInEachLocalSpace",
			new IMAGE_BRUSH_SVG("Starship/EditorViewport/LocalTransformsLocalSpace_16", Icon16x16));
		Set("EditorViewport.OnlySelectRigControls",
			new IMAGE_BRUSH_SVG("Starship/Animation/AnimationSelectOnlyControlRig_16", Icon16x16));

		Set( "EditorViewport.CamSpeedSetting", new IMAGE_BRUSH_SVG( "Starship/EditorViewport/camera", Icon16x16) );
		
		Set( "EditorViewport.LitMode",            	  new IMAGE_BRUSH_SVG("Starship/Common/LitCube", Icon16x16 ) );
		Set( "EditorViewport.UnlitMode",          	  new IMAGE_BRUSH_SVG("Starship/Common/UnlitCube", Icon16x16 ) );
		Set( "EditorViewport.WireframeMode",      	  new IMAGE_BRUSH_SVG("Starship/Common/BrushWireframe", Icon16x16 ) );
		Set( "EditorViewport.LitWireframeMode",       new IMAGE_BRUSH_SVG("Starship/Common/LitWireframe_16", Icon16x16) );
		Set( "EditorViewport.DetailLightingMode", 	  new IMAGE_BRUSH_SVG("Starship/Common/DetailLighting", Icon16x16 ) );
		Set( "EditorViewport.LightingOnlyMode",   	  new IMAGE_BRUSH_SVG("Starship/Common/LightBulb", Icon16x16 ) );
		
		Set( "EditorViewport.PathTracingMode",   	  new IMAGE_BRUSH_SVG("Starship/Common/PathTracing", Icon16x16 ) );
		Set( "EditorViewport.RayTracingDebugMode",    new IMAGE_BRUSH_SVG("Starship/Common/RayTracingDebug", Icon16x16 ) );

		Set( "EditorViewport.LightComplexityMode", new IMAGE_BRUSH( "Icons/icon_ViewMode_LightComplexity_16px", Icon16x16 ) );
		Set( "EditorViewport.ShaderComplexityMode", new IMAGE_BRUSH( "Icons/icon_ViewMode_Shadercomplexity_16px", Icon16x16 ) );

		Set( "EditorViewport.QuadOverdrawMode", new IMAGE_BRUSH_SVG("Starship/Common/OptimizationViewmodes", Icon16x16 ) );

		Set( "EditorViewport.GeometryInspection", new IMAGE_BRUSH_SVG( "Starship/Common/GeometryInspection_16", Icon16x16 ) );

		Set( "EditorViewport.ShaderComplexityWithQuadOverdrawMode", new IMAGE_BRUSH( "Icons/icon_ViewMode_Shadercomplexity_16px", Icon16x16 ) );
		Set( "EditorViewport.TexStreamAccPrimitiveDistanceMode", new IMAGE_BRUSH( "Icons/icon_ViewMode_TextureStreamingAccuracy_16px", Icon16x16 ) );
		Set( "EditorViewport.TexStreamAccMeshUVDensityMode", new IMAGE_BRUSH("Icons/icon_ViewMode_TextureStreamingAccuracy_16px", Icon16x16));
		Set( "EditorViewport.TexStreamAccMaterialTextureScaleMode", new IMAGE_BRUSH( "Icons/icon_ViewMode_TextureStreamingAccuracy_16px", Icon16x16 ) );
		Set( "EditorViewport.RequiredTextureResolutionMode", new IMAGE_BRUSH( "Icons/icon_ViewMode_TextureStreamingAccuracy_16px", Icon16x16 ) );
		Set( "EditorViewport.StationaryLightOverlapMode", new IMAGE_BRUSH( "Icons/icon_ViewMode_StationaryLightOverlap_16px", Icon16x16 ) );
		Set( "EditorViewport.LightmapDensityMode", new IMAGE_BRUSH( "Icons/icon_ViewMode_LightmapDensity_16px", Icon16x16 ) );

		Set( "EditorViewport.LODColorationMode", new IMAGE_BRUSH("Icons/icon_ViewMode_LODColoration_16px", Icon16x16) );
		Set( "EditorViewport.HLODColorationMode", new IMAGE_BRUSH("Icons/icon_ViewMode_LODColoration_16px", Icon16x16));	
		Set( "EditorViewport.GroupLODColorationMode", new IMAGE_BRUSH_SVG("Starship/Common/LODColorization", Icon16x16) );
		Set( "EditorViewport.VisualizeGPUSkinCacheMode", new IMAGE_BRUSH_SVG("Starship/Common/SkeletalMesh", Icon16x16));
		Set( "EditorViewport.LWCComplexityMode", new IMAGE_BRUSH( "Icons/icon_ViewMode_Shadercomplexity_16px", Icon16x16 ) );

		Set( "EditorViewport.VisualizeGBufferMode",   new IMAGE_BRUSH_SVG("Starship/Common/BufferVisualization", Icon16x16) );

		Set( "EditorViewport.Visualizers", 			  new CORE_IMAGE_BRUSH_SVG("Starship/Common/Visualizer", Icon16x16) );
		Set( "EditorViewport.LOD", 			  		  new IMAGE_BRUSH_SVG("Starship/Common/LOD", Icon16x16) );

		Set( "EditorViewport.ReflectionOverrideMode", new IMAGE_BRUSH_SVG("Starship/Common/Reflections", Icon16x16 ) );
		Set( "EditorViewport.VisualizeBufferMode",    new IMAGE_BRUSH_SVG("Starship/Common/BufferVisualization", Icon16x16 ) );
		Set( "EditorViewport.VisualizeNaniteMode",    new IMAGE_BRUSH_SVG("Starship/Common/Nanite_16", Icon16x16 ) );
		Set( "EditorViewport.VisualizeLumenMode",     new IMAGE_BRUSH_SVG("Starship/Common/Lumen_16", Icon16x16 ) );
		Set( "EditorViewport.VisualizeMegaLightsMode", new IMAGE_BRUSH_SVG("Starship/Common/MegaLights_16", Icon16x16 ) );
		Set( "EditorViewport.VisualizeSubstrateMode", new IMAGE_BRUSH_SVG("Starship/Common/BufferVisualization", Icon16x16 ) );
		Set( "EditorViewport.VisualizeGroomMode",     new IMAGE_BRUSH("Common/icon_ShowHair_16x", Icon16x16 ) );
		Set( "EditorViewport.VisualizeVirtualShadowMapMode", new IMAGE_BRUSH_SVG("Starship/Common/VirtualShadowMap_16", Icon16x16 ) );
		Set( "EditorViewport.VisualizeActorColorationMode", new IMAGE_BRUSH_SVG("Starship/Common/ActorColoration_16", Icon16x16));
		Set( "EditorViewport.VisualizeVirtualTextureMode", new IMAGE_BRUSH("Icons/AssetIcons/Texture2D_16x", Icon16x16));
		Set( "EditorViewport.CollisionPawn",          new IMAGE_BRUSH_SVG("Starship/Common/PlayerCollision", Icon16x16 ) );
		Set( "EditorViewport.CollisionVisibility",    new IMAGE_BRUSH_SVG("Starship/Common/VisibilityCollision", Icon16x16 ) );
		Set( "EditorViewport.MeshEdges",              new IMAGE_BRUSH_SVG("Starship/Common/MeshEdges_16", Icon16x16 ) );

		Set( "EditorViewport.AllChannelsMask",   new IMAGE_BRUSH_SVG("Starship/Common/ColorChannelRGB", Icon16x16 ) );
		Set( "EditorViewport.RedChannelMask",   new IMAGE_BRUSH_SVG("Starship/Common/ColorChannelRed", Icon16x16 ) );
		Set( "EditorViewport.GreenChannelMask", new IMAGE_BRUSH_SVG("Starship/Common/ColorChannelGreen", Icon16x16 ) );
		Set( "EditorViewport.BlueChannelMask",  new IMAGE_BRUSH_SVG("Starship/Common/ColorChannelBlue", Icon16x16 ) );
		Set( "EditorViewport.AlphaChannelMask", new IMAGE_BRUSH_SVG("Starship/Common/ColorChannelAlpha", Icon16x16 ) );

		Set( "EditorViewport.Perspective", new IMAGE_BRUSH_SVG("Starship/Common/ViewPerspective", Icon16x16 ) );
		Set( "EditorViewport.Top",         new IMAGE_BRUSH_SVG("Starship/Common/ViewTop", Icon16x16 ) );
		Set( "EditorViewport.Left",        new IMAGE_BRUSH_SVG("Starship/Common/ViewLeft", Icon16x16 ) );
		Set( "EditorViewport.Front",       new IMAGE_BRUSH_SVG("Starship/Common/ViewFront", Icon16x16 ) );
		Set( "EditorViewport.Bottom",      new IMAGE_BRUSH_SVG("Starship/Common/ViewBottom", Icon16x16 ) );
		Set( "EditorViewport.Right",       new IMAGE_BRUSH_SVG("Starship/Common/ViewRight", Icon16x16 ) );
		Set( "EditorViewport.Back",        new IMAGE_BRUSH_SVG("Starship/Common/ViewBack", Icon16x16 ) );
		
		Set( "EditorViewport.UseOrthographicClippingPlanes", new IMAGE_BRUSH_SVG("Starship/Common/MediaPlanes", Icon16x16) );
		Set( "EditorViewport.OrthographicClippingSettings", new IMAGE_BRUSH_SVG("Starship/Common/tool-properties", Icon16x16) );
		Set( "EditorViewport.DismissOrthographicClippingPlanes", new IMAGE_BRUSH_SVG("Starship/Common/cross-16", Icon16x16) );

		Set("EditorViewport.ToggleStats", new IMAGE_BRUSH_SVG("Starship/Common/Statistics", Icon16x16));
		Set("EditorViewport.ToggleFPS", new IMAGE_BRUSH_SVG("Starship/Common/FPS", Icon16x16));
		Set("EditorViewport.ToggleViewportToolbar", new IMAGE_BRUSH_SVG("Starship/Common/Toolbar", Icon16x16));
		Set("EditorViewport.ToggleDistanceBasedCameraSpeed", new IMAGE_BRUSH_SVG("Starship/Common/CameraSpeed_16", Icon16x16));

		Set("EditorViewport.SubMenu.Stats", new IMAGE_BRUSH_SVG("Starship/Common/Statistics", Icon16x16));
		Set("EditorViewport.SubMenu.Bookmarks", new IMAGE_BRUSH_SVG("Starship/Common/Bookmarks", Icon16x16));
		Set("EditorViewport.SubMenu.CreateCamera", new IMAGE_BRUSH_SVG("Starship/Common/CreateCamera", Icon16x16));

		Set("LevelViewport.ToggleGameView", new IMAGE_BRUSH_SVG("Starship/Common/GameView", Icon16x16));
		Set("LevelViewport.ToggleImmersive", new IMAGE_BRUSH_SVG("Starship/Common/EnableFullscreen", Icon16x16));
		Set("LevelViewport.HighResScreenshot", new IMAGE_BRUSH_SVG("Starship/Common/HighResolutionScreenshot", Icon16x16));
		Set("LevelViewport.ToggleCinematicPreview", new IMAGE_BRUSH_SVG("Starship/Common/Cinematics", Icon16x16));
		Set("LevelViewport.ToggleAllowConstrainedAspectRatioInPreview", new IMAGE_BRUSH_SVG("Starship/Common/ConstrainedAspectRatio_16", Icon16x16));
		Set("LevelViewport.ToggleFlipConstrainedAspectRatioInPreview", new IMAGE_BRUSH_SVG("Starship/Common/RotationScreenspace_16", Icon16x16));
		Set("LevelViewport.ToggleCameraShakePreview", new IMAGE_BRUSH_SVG("Starship/Common/CameraShake", Icon16x16));
		Set("LevelViewport.AdvancedSettings", new CORE_IMAGE_BRUSH_SVG("Starship/Common/settings", Icon16x16));
		Set("LevelViewport.PlaySettings", new CORE_IMAGE_BRUSH_SVG("Starship/Common/settings", Icon16x16));

		Set("EditorViewport.ActiveBorderColor", FStyleColors::Primary);

#endif

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
		{
			Set( "LevelEditor.Tabs.Details",                new IMAGE_BRUSH_SVG("Starship/Common/Details", Icon16x16) ); // Use Icons.Details instead of this
			Set( "LevelEditor.Tabs.Cinematics",             new IMAGE_BRUSH_SVG("Starship/Common/Cinematics", Icon16x16) );
			Set( "LevelEditor.Tabs.Media",                  new IMAGE_BRUSH_SVG("Starship/AssetIcons/MediaPlayer_16", Icon16x16) );
			Set( "LevelEditor.Tabs.VirtualProduction",      new IMAGE_BRUSH_SVG("Starship/Common/VirtualProduction", Icon16x16) );
			Set( "LevelEditor.Tabs.EditorModes",            new IMAGE_BRUSH_SVG("Starship/Common/EditorModes", Icon16x16) );
			Set( "LevelEditor.Tabs.Modes",                  new IMAGE_BRUSH_SVG("Starship/Common/EditorModes", Icon16x16) );
			Set( "LevelEditor.Tabs.PlacementBrowser",       new IMAGE_BRUSH_SVG("Starship/Common/PlaceActors", Icon16x16) );
			Set( "LevelEditor.Tabs.Properties",             new IMAGE_BRUSH_SVG("Starship/StaticMeshEditor/Properties", Icon16x16) );
			Set( "LevelEditor.Tabs.Outliner",               new IMAGE_BRUSH_SVG("Starship/Common/WorldOutliner", Icon16x16) );
			Set( "LevelEditor.Tabs.ContentBrowser",         new IMAGE_BRUSH_SVG("Starship/Common/ContentBrowser", Icon16x16) );

			Set( "LevelEditor.Tabs.Levels",                 new IMAGE_BRUSH_SVG("Starship/WorldBrowser/LevelStack", Icon16x16) );
			Set( "LevelEditor.Tabs.WorldBrowser",           new IMAGE_BRUSH_SVG("Starship/WorldBrowser/LevelStack", Icon16x16) );
			Set( "LevelEditor.Tabs.WorldBrowserDetails",    new IMAGE_BRUSH_SVG("Starship/Common/Details", Icon16x16) );

			Set( "LevelEditor.Tabs.WorldBrowserComposition",new IMAGE_BRUSH_SVG("Starship/WorldBrowser/WorldComp_16", Icon16x16 ) );
			Set( "LevelEditor.Tabs.WorldPartition",			new IMAGE_BRUSH( "/Icons/icon_levels_partitionbutton_16x", Icon16x16 ) );

			Set( "LevelEditor.Tabs.Layers",                 new IMAGE_BRUSH_SVG("Starship/Common/Layers", Icon16x16) );
			Set( "LevelEditor.Tabs.DataLayers",				new IMAGE_BRUSH_SVG("Starship/Common/DataLayers", Icon16x16));
			Set( "LevelEditor.Tabs.ComposureCompositing",   new IMAGE_BRUSH_SVG("Starship/Common/ComposureCompositing", Icon16x16) );
			Set( "LevelEditor.Tabs.USDStage",   			new IMAGE_BRUSH_SVG("Starship/Common/USDStage", Icon16x16) );

			Set( "LevelEditor.Tabs.StatsViewer",            new IMAGE_BRUSH_SVG("Starship/Common/Statistics", Icon16x16) );

			Set( "LevelEditor.Tabs.Toolbar",                new IMAGE_BRUSH("Icons/icon_tab_Toolbars_16x", Icon16x16 ) );

			Set( "LevelEditor.Tabs.Viewports",              new IMAGE_BRUSH_SVG("Starship/Common/Viewports", Icon16x16) );
			Set( "LevelEditor.Tabs.HLOD",                   new IMAGE_BRUSH_SVG("Starship/Common/HierarchicalLOD", Icon16x16) );
			Set( "LevelEditor.Tabs.Debug",                  new IMAGE_BRUSH_SVG("Starship/Common/Bug", Icon16x16) );

			Set( "LevelEditor.Audit",                  		new IMAGE_BRUSH_SVG("Starship/Common/AssetAudit", Icon16x16) );
			Set( "LevelEditor.Profile",                  	new IMAGE_BRUSH_SVG("Starship/Common/Profile", Icon16x16) );
			Set( "LevelEditor.Platforms",                  	new IMAGE_BRUSH_SVG("Starship/Common/DeviceManager", Icon16x16) );
		}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR
		Set( "LevelEditor.NewLevel",      new IMAGE_BRUSH_SVG( "Starship/Common/LevelNew", Icon16x16 ) );
		Set( "SystemWideCommands.OpenLevel",     new IMAGE_BRUSH_SVG( "Starship/Common/LevelOpen", Icon16x16 ) );
		Set( "LevelEditor.Save",          new IMAGE_BRUSH_SVG( "Starship/Common/SaveCurrent", Icon16x16 ) );
		Set( "LevelEditor.SaveAs",        new IMAGE_BRUSH_SVG( "Starship/Common/SaveCurrentAs", Icon16x16 ) );
		Set( "LevelEditor.SaveAllLevels", new IMAGE_BRUSH_SVG( "Starship/Common/LevelSaveAll", Icon16x16 ) );

		Set( "LevelEditor.ImportScene",    new IMAGE_BRUSH_SVG( "Starship/Common/LevelImportInto", Icon16x16 ) );
		Set( "LevelEditor.ExportAll",      new CORE_IMAGE_BRUSH_SVG( "Starship/Common/export", Icon16x16 ) );
		Set( "LevelEditor.ExportSelected", new IMAGE_BRUSH_SVG( "Starship/Common/ExportSelected", Icon16x16 ) );

		Set( "LevelEditor.Recompile", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/compile", Icon40x40 ) );
		Set( "LevelEditor.Recompile.Small", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/compile", Icon20x20 ) );

		Set("LevelEditor.PreviewMode.Enabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_SM5_Enabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.Disabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_SM5_Disabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.SM5.Enabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_SM5_Enabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.SM5.Disabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_SM5_Enabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.AndroidES31.Enabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_AndroidES31_Enabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.AndroidES31.Disabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_AndroidES31_Disabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.AndroidVulkan.Enabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_AndroidVulkan_Enabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.AndroidVulkan.Disabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_AndroidVulkan_Disabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.AndroidVulkanSM5.Enabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_AndroidVulkanSM5_Enabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.AndroidVulkanSM5.Disabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_AndroidVulkanSM5_Disabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.iOS.Enabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_iOS_Enabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.iOS.Disabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_iOS_Disabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.iOSSM5.Enabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_iOSSM5_Enabled_40x", Icon40x40));
		Set("LevelEditor.PreviewMode.iOSSM5.Disabled", new IMAGE_BRUSH("Icons/icon_PreviewMode_iOSSM5_Disabled_40x", Icon40x40));

		Set("LevelEditor.ViewOptions", new IMAGE_BRUSH("Icons/icon_view_40x", Icon40x40));
		Set( "LevelEditor.ViewOptions.Small", new IMAGE_BRUSH( "Icons/icon_view_40x", Icon20x20 ) );

		Set( "LevelEditor.GameSettings", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/settings", Icon20x20) );

		Set( "LevelEditor.Create", new IMAGE_BRUSH( "Icons/icon_Mode_Placement_40px", Icon40x40 ) );
		Set( "LevelEditor.Create.Small", new IMAGE_BRUSH( "Icons/icon_Mode_Placement_40px", Icon20x20 ) );
		Set( "LevelEditor.Create.OutlineHoriz", new IMAGE_BRUSH( "Common/WorkingFrame_Marquee", FVector2f(34.0f, 3.0f), FLinearColor::White, ESlateBrushTileType::Horizontal) );
		Set( "LevelEditor.Create.OutlineVert", new IMAGE_BRUSH( "Common/WorkingFrame_Marquee_Vert", FVector2f(3.0f, 34.0f), FLinearColor::White, ESlateBrushTileType::Vertical) );

		Set( "LevelEditor.Tab", new IMAGE_BRUSH_SVG("Starship/AssetIcons/World_16", Icon16x16));
		Set( "LevelEditor.AssetColor", FColor(255, 156, 0));

		Set( "ToolPalette.DockingTab", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetPadding( FMargin(16.0f, 2.0f, 16.0f, 2.0f ) )
			.SetCheckedImage(         CORE_BOX_BRUSH("Docking/Tab_Shape",  2.f/8.0f, FLinearColor(FColor(62, 62, 62)) ) )
			.SetCheckedHoveredImage(  CORE_BOX_BRUSH("Docking/Tab_Shape",  2.f/8.0f, FLinearColor(FColor(62, 62, 62)) ) )
			.SetCheckedPressedImage(  CORE_BOX_BRUSH("Docking/Tab_Shape",  2.f/8.0f, FLinearColor(FColor(62, 62, 62)) ) )
			.SetUncheckedImage(       CORE_BOX_BRUSH("Docking/Tab_Shape",  2.f/8.0f, FLinearColor(FColor(45, 45, 45)) ) )
			.SetUncheckedHoveredImage(CORE_BOX_BRUSH("Docking/Tab_Shape",2.f/8.0f, FLinearColor(FColor(54, 54, 54)) ) )
			.SetUncheckedPressedImage(CORE_BOX_BRUSH("Docking/Tab_Shape",2.f/8.0f, FLinearColor(FColor(54, 54, 54)) ) )
			.SetUndeterminedImage(        FSlateNoResource() )
			.SetUndeterminedHoveredImage( FSlateNoResource() )
			.SetUndeterminedPressedImage( FSlateNoResource() )
		);	
		Set( "ToolPalette.DockingWell", new FSlateColorBrush(FLinearColor(FColor(34, 34, 34, 255))));

		Set( "ToolPalette.DockingLabel", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 9 ) ) 
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f))
		);


		Set("LevelEditor.SelectMode", new IMAGE_BRUSH_SVG("Starship/MainToolbar/select", Icon20x20));

		Set( "LevelEditor.MeshPaintMode", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/paint", Icon20x20 ) );
		
		Set("LevelEditor.MeshPaintMode.TexturePaint", new IMAGE_BRUSH("Icons/TexturePaint_40x", Icon40x40));
		Set("LevelEditor.MeshPaintMode.TexturePaint.Small", new IMAGE_BRUSH("Icons/TexturePaint_40x", Icon20x20));
		Set("LevelEditor.MeshPaintMode.ColorPaint", new IMAGE_BRUSH("Icons/VertexColorPaint_40x", Icon40x40));
		Set("LevelEditor.MeshPaintMode.ColorPaint.Small", new IMAGE_BRUSH("Icons/VertexColorPaint_40x", Icon20x20));
		Set("LevelEditor.MeshPaintMode.WeightPaint", new IMAGE_BRUSH("Icons/WeightPaint_40x", Icon40x40));
		Set("LevelEditor.MeshPaintMode.WeightPaint.Small", new IMAGE_BRUSH("Icons/WeightPaint_40x", Icon20x20));

		Set( "LevelEditor.LandscapeMode", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/landscape", Icon20x20 ) );
		Set( "LevelEditor.LandscapeMode.Selected", new IMAGE_BRUSH( "Icons/icon_Mode_Landscape_selected_40x", Icon40x40 ) );
		Set( "LevelEditor.LandscapeMode.Selected.Small", new IMAGE_BRUSH( "Icons/icon_Mode_Landscape_selected_40x", Icon20x20 ) );

		Set( "LevelEditor.FoliageMode", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/foliage", Icon20x20 ) );
		Set( "LevelEditor.FoliageMode.Selected", new IMAGE_BRUSH( "Icons/icon_Mode_Foliage_selected_40x", Icon40x40 ) );
		Set( "LevelEditor.FoliageMode.Selected.Small", new IMAGE_BRUSH( "Icons/icon_Mode_Foliage_selected_40x", Icon20x20 ) );

		Set( "LevelEditor.WorldProperties", new IMAGE_BRUSH( "Icons/icon_worldscript_40x", Icon40x40 ) );
		Set( "LevelEditor.WorldProperties.Small", new IMAGE_BRUSH( "Icons/icon_worldscript_40x", Icon20x20 ) );

		Set( "LevelEditor.WorldProperties.Tab", new IMAGE_BRUSH_SVG( "Starship/Common/WorldSettings", Icon16x16 ) );

		Set("LevelEditor.BrushEdit", new IMAGE_BRUSH_SVG("Starship/MainToolbar/brush_edit", Icon20x20));

		Set( "LevelEditor.OpenPlaceActors", new IMAGE_BRUSH_SVG( "Starship/Common/PlaceActors", Icon20x20 ) );
		Set( "LevelEditor.OpenContentBrowser", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/content", Icon20x20 ) );
		Set( "LevelEditor.OpenMarketplace", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/marketplace", Icon20x20) );
		Set( "LevelEditor.ImportContent", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import", Icon20x20));
		Set( "LevelEditor.CreateBlankBlueprintClass", new IMAGE_BRUSH_SVG("Starship/MainToolbar/blueprints", Icon20x20));
		Set( "LevelEditor.ConvertSelectionToBlueprint", new IMAGE_BRUSH_SVG("Starship/Common/convert", Icon20x20));
		Set( "LevelEditor.OpenLevelBlueprint", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/LevelBlueprint", Icon20x20) );
		Set( "LevelEditor.OpenCinematic", new IMAGE_BRUSH_SVG("Starship/MainToolbar/cinematics", Icon20x20));

		Set( "LevelEditor.OpenAddContent.Background", new IMAGE_BRUSH_SVG("Starship/MainToolbar/PlaceActorsBase", Icon20x20));
		Set( "LevelEditor.OpenAddContent.Overlay", new IMAGE_BRUSH_SVG("Starship/MainToolbar/ToolBadgePlus", Icon20x20, FStyleColors::AccentGreen));

		Set( "LevelEditor.CreateClassBlueprint", new IMAGE_BRUSH("Icons/icon_class_Blueprint_New_16x", Icon16x16));
		Set( "LevelEditor.OpenClassBlueprint", new IMAGE_BRUSH_SVG("Starship/Common/BrowseContent", Icon16x16));

		Set( "LevelEditor.ToggleVR", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/VRTools", Icon40x40 ) );
		Set( "LevelEditor.ToggleVR.Small", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/VRTools", Icon20x20 ) );

		Set( "MergeActors.MeshMergingTool", new IMAGE_BRUSH_SVG( "Starship/MergeActors/MeshMerging_16", Icon16x16 ) );
		Set( "MergeActors.MeshProxyTool", new IMAGE_BRUSH_SVG( "Starship/MergeActors/MeshProxy_16", Icon16x16 ) );
		Set( "MergeActors.MeshInstancingTool", new IMAGE_BRUSH_SVG("Starship/AssetIcons/StaticMeshActor_16", Icon16x16 ) );
		Set( "MergeActors.TabIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Merge", Icon16x16));
		Set( "MergeActors.Approximate", new IMAGE_BRUSH_SVG("Starship/MergeActors/approximate", Icon16x16));

		// Top level Actors Menu
		Set( "Actors.Attach", new IMAGE_BRUSH_SVG("Starship/Actors/attach", Icon16x16));
		Set( "Actors.Detach", new IMAGE_BRUSH_SVG("Starship/Actors/detach", Icon16x16));
		Set( "Actors.TakeRecorder", new IMAGE_BRUSH_SVG("Starship/Actors/take-recorder", Icon16x16));
		Set( "Actors.GoHere", new IMAGE_BRUSH_SVG("Starship/Actors/go-here", Icon16x16));
		Set( "Actors.SnapViewToObject", new IMAGE_BRUSH_SVG("Starship/Actors/snap-view-to-object", Icon16x16));
		Set( "Actors.SnapObjectToView", new IMAGE_BRUSH_SVG("Starship/Actors/snap-object-to-view", Icon16x16));
		Set( "Actors.ScripterActorActions", new IMAGE_BRUSH_SVG("Starship/Actors/scripted-actor-actions", Icon16x16));

		Set( "PlacementBrowser.OptionsMenu", new IMAGE_BRUSH( "Icons/icon_Blueprint_Macro_16x", Icon16x16 ) );

		Set( "PlacementBrowser.AssetToolTip.AssetName", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Bold", 9 ) ) );
		Set( "PlacementBrowser.AssetToolTip.AssetClassName", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 9 ) ) );
		Set( "PlacementBrowser.AssetToolTip.AssetPath", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 8 ) ) );

		Set( "PlacementBrowser.Asset", FButtonStyle( Button )
			.SetNormal( FSlateRoundedBoxBrush(FLinearColor::Transparent, 6.0f, FStyleColors::Dropdown, 1.0f) )
			.SetHovered( FSlateRoundedBoxBrush(FLinearColor::Transparent, 6.0f, FStyleColors::Hover, 1.0f) )
			.SetPressed( FSlateRoundedBoxBrush(FLinearColor::Transparent, 6.0f, FStyleColors::Primary, 1.0f) )
			.SetNormalPadding( 0 )
			.SetPressedPadding( 0 )
			);

		Set( "PlacementBrowser.Asset.Background", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 6.f));
		Set( "PlacementBrowser.Asset.LabelBack", new BOX_BRUSH("Starship/PlacementBrowser/LabelBack_18x", 6.f/18.f, FStyleColors::Dropdown));
		Set( "PlacementBrowser.Asset.ThumbnailBackground", new FSlateRoundedBoxBrush(FStyleColors::Dropdown, 6.f ));

		FLinearColor DimBackground = FLinearColor( FColor( 64, 64, 64 ) );
		FLinearColor DimBackgroundHover = FLinearColor( FColor( 50, 50, 50 ) );
		FLinearColor DarkBackground = FLinearColor( FColor( 42, 42, 42 ) );

		Set( "PlacementBrowser.Tab", FCheckBoxStyle()
			.SetCheckBoxType( ESlateCheckBoxType::ToggleButton )
			.SetUncheckedImage( BOX_BRUSH( "Common/Selection", 8.0f / 32.0f, DimBackground ) )
			.SetUncheckedPressedImage( BOX_BRUSH( "PlacementMode/TabActive", 8.0f / 32.0f ) )
			.SetUncheckedHoveredImage( BOX_BRUSH( "Common/Selection", 8.0f / 32.0f, DimBackgroundHover ) )
			.SetCheckedImage( BOX_BRUSH( "PlacementMode/TabActive", 8.0f / 32.0f ) )
			.SetCheckedHoveredImage( BOX_BRUSH( "PlacementMode/TabActive", 8.0f / 32.0f ) )
			.SetCheckedPressedImage( BOX_BRUSH( "PlacementMode/TabActive", 8.0f / 32.0f ) )
			.SetPadding( 0 ) );

		Set( "PlacementBrowser.Tab.Text", FTextBlockStyle( NormalText )
			.SetFont( DEFAULT_FONT( "Bold", 10 ) )
			.SetColorAndOpacity( FLinearColor( 1.0f, 1.0f, 1.0f, 0.9f ) )
			.SetShadowOffset( FVector2f::UnitVector )
			.SetShadowColorAndOpacity( FLinearColor( 0.f, 0.f, 0.f, 0.9f ) ) );

		Set( "PlacementBrowser.Asset.Name", FTextBlockStyle( NormalText )
			.SetFont( DEFAULT_FONT( "Regular", 10 ) )
			.SetColorAndOpacity( FLinearColor( 1.0f, 1.0f, 1.0f, 0.9f ) ) );

		Set( "PlacementBrowser.Asset.Type", FTextBlockStyle( NormalText )
			.SetFont( DEFAULT_FONT( "Regular", 8 ) )
			.SetColorAndOpacity( FLinearColor( 0.8f, 0.8f, 0.8f, 0.9f ) )
			.SetShadowOffset( FVector2f::UnitVector )
			.SetShadowColorAndOpacity( FLinearColor( 0.f, 0.f, 0.f, 0.9f ) ) );

		Set( "PlacementBrowser.ActiveTabNub", new IMAGE_BRUSH( "Icons/TabTriangle_24x", Icon24x24, FLinearColor( FColor( 42, 42, 42 ) ) ) );
		Set( "PlacementBrowser.ActiveTabBar", new IMAGE_BRUSH( "Common/Selection", FVector2f(2.0f, 2.0f), SelectionColor ) );

		Set( "PlacementBrowser.ShowAllContent", new IMAGE_BRUSH( "Icons/icon_Placement_AllContent_20px", Icon20x20 ) );
		Set( "PlacementBrowser.ShowAllContent.Small", new IMAGE_BRUSH( "Icons/icon_Placement_AllContent_20px", Icon20x20 ) );
		Set( "PlacementBrowser.ShowCollections", new IMAGE_BRUSH( "Icons/icon_Placement_Collections_20px", Icon20x20 ) );
		Set( "PlacementBrowser.ShowCollections.Small", new IMAGE_BRUSH( "Icons/icon_Placement_Collections_20px", Icon20x20 ) );


		const FTableRowStyle PlaceItemTableRowStyle = FTableRowStyle()
			.SetEvenRowBackgroundBrush(FSlateNoResource())
			.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())

			.SetOddRowBackgroundBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(FSlateNoResource())

			.SetSelectorFocusedBrush(BORDER_BRUSH("Common/Selector", FMargin(4.f / 16.f), SelectorColor))

			.SetActiveBrush(FSlateNoResource())
			.SetActiveHoveredBrush(FSlateNoResource())

			.SetInactiveBrush(FSlateNoResource())
			.SetInactiveHoveredBrush(FSlateNoResource())

			.SetActiveHighlightedBrush(FSlateNoResource())
			.SetInactiveHighlightedBrush(FSlateNoResource())

			.SetTextColor(FStyleColors::Foreground)
			.SetSelectedTextColor(FStyleColors::Foreground)

			.SetDropIndicator_Above(BOX_BRUSH("Common/DropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0, 0), SelectionColor))
			.SetDropIndicator_Onto(BOX_BRUSH("Common/DropZoneIndicator_Onto", FMargin(4.0f / 16.0f), SelectionColor))
			.SetDropIndicator_Below(BOX_BRUSH("Common/DropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), SelectionColor));

		Set("PlacementBrowser.PlaceableItemRow", PlaceItemTableRowStyle);


		const FCheckBoxStyle PlacementSegmentedBox = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(FSlateNoResource())
			.SetUncheckedHoveredImage(FSlateNoResource())
			.SetUncheckedPressedImage(FSlateNoResource())
			.SetCheckedImage(FSlateNoResource())
			.SetCheckedHoveredImage(FSlateNoResource())
			.SetCheckedPressedImage(FSlateNoResource())
			.SetForegroundColor(FStyleColors::Foreground)
			.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedForegroundColor(FStyleColors::Primary)
			.SetCheckedHoveredForegroundColor(FStyleColors::Primary)
			.SetCheckedPressedForegroundColor(FStyleColors::Primary)
			.SetPadding(FMargin(6.f, 2.f));

		Set("PlacementBrowser.CategoryControl", FSegmentedControlStyle()
			.SetControlStyle(PlacementSegmentedBox)
			.SetFirstControlStyle(PlacementSegmentedBox)
			.SetLastControlStyle(PlacementSegmentedBox)
		);

		Set("PlacementBrowser.Icons.Recent",        new CORE_IMAGE_BRUSH_SVG("Starship/Common/Recent",    Icon20x20));
		Set("PlacementBrowser.Icons.Basic",         new IMAGE_BRUSH_SVG("Starship/Common/Basic",          Icon20x20));
		Set("PlacementBrowser.Icons.Lights",        new IMAGE_BRUSH_SVG("Starship/Common/LightBulb",      Icon20x20));
		Set("PlacementBrowser.Icons.Cinematics",    new IMAGE_BRUSH_SVG("Starship/Common/Cinematics",     Icon20x20));
		Set("PlacementBrowser.Icons.VisualEffects", new IMAGE_BRUSH_SVG("Starship/Common/VisualEffects",  Icon20x20));
		Set("PlacementBrowser.Icons.BSP",           new IMAGE_BRUSH_SVG("Starship/Common/Geometry",       Icon20x20));
		Set("PlacementBrowser.Icons.Volumes",       new IMAGE_BRUSH_SVG("Starship/Common/Volumes",        Icon20x20));
		Set("PlacementBrowser.Icons.All",           new IMAGE_BRUSH_SVG("Starship/Common/AllClasses",     Icon20x20));
		Set("PlacementBrowser.Icons.Testing",       new CORE_IMAGE_BRUSH_SVG("Starship/Common/Test",      Icon20x20));
		Set("PlacementBrowser.Icons.Shapes",        new IMAGE_BRUSH_SVG("Starship/Common/Shapes",         Icon20x20));

		Set( "ContentPalette.ShowAllPlaceables", new IMAGE_BRUSH( "Icons/icon_Placement_FilterAll_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowAllPlaceables.Small", new IMAGE_BRUSH( "Icons/icon_Placement_FilterAll_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowProps", new IMAGE_BRUSH( "Icons/icon_Placement_FilterProps_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowProps.Small", new IMAGE_BRUSH( "Icons/icon_Placement_FilterProps_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowParticles", new IMAGE_BRUSH( "Icons/icon_Placement_FilterParticles_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowParticles.Small", new IMAGE_BRUSH( "Icons/icon_Placement_FilterParticles_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowAudio", new IMAGE_BRUSH( "Icons/icon_Placement_FilterAudio_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowAudio.Small", new IMAGE_BRUSH( "Icons/icon_Placement_FilterAudio_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowMisc", new IMAGE_BRUSH( "Icons/icon_Placement_FilterMisc_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowMisc.Small", new IMAGE_BRUSH( "Icons/icon_Placement_FilterMisc_20px", Icon20x20 ) );
		Set( "ContentPalette.ShowRecentlyPlaced", new IMAGE_BRUSH( "Icons/icon_Placement_RecentlyPlaced_20x", Icon20x20 ) );
		Set( "ContentPalette.ShowRecentlyPlaced.Small", new IMAGE_BRUSH( "Icons/icon_Placement_RecentlyPlaced_20x", Icon20x20 ) );
	}

	{

		Set( "AssetDeleteDialog.Background", new IMAGE_BRUSH( "Common/Selection", Icon8x8, FLinearColor( 0.016f, 0.016f, 0.016f ) ) );
	}

	// Level editor tool box icons
	{
		Set( "LevelEditor.RecompileGameCode", new IMAGE_BRUSH( "Old/MainToolBar/RecompileGameCode", Icon40x40 ) );
	}

	// Editor viewport layout command icons
	{
		const FVector2f IconLayoutSize(47.0f, 37.0f);
		const FVector2f IconLayoutSizeSmall(47.0f, 37.0f);		// small version set to same size as these are in their own menu and don't clutter the UI

		Set("EditorViewport.ViewportConfig_OnePane", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts1Pane", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_OnePane.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts1Pane", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_TwoPanesH", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts2Panes", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_TwoPanesH.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts2Panes", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_TwoPanesV", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts2PanesStacked", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_TwoPanesV.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts2PanesStacked", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_ThreePanesLeft", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesLeft", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_ThreePanesLeft.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesLeft", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_ThreePanesRight", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesRight", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_ThreePanesRight.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesRight", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_ThreePanesTop", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesTop", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_ThreePanesTop.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesTop", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_ThreePanesBottom", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesBottom", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_ThreePanesBottom.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesBottom", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_FourPanesLeft", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesLeft", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_FourPanesLeft.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesLeft", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_FourPanesRight", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesRight", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_FourPanesRight.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesRight", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_FourPanesTop", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesTop", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_FourPanesTop.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesTop", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_FourPanesBottom", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesBottom", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_FourPanesBottom.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesBottom", IconLayoutSizeSmall));
		Set("EditorViewport.ViewportConfig_FourPanes2x2", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4Panes", IconLayoutSize));
		Set("EditorViewport.ViewportConfig_FourPanes2x2.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4Panes", IconLayoutSizeSmall));
	}

	// Level viewport layout command icons
	{
		const FVector2f IconLayoutSize(47.0f, 37.0f);
		const FVector2f IconLayoutSizeSmall(47.0f, 37.0f);		// small version set to same size as these are in their own menu and don't clutter the UI

		Set("LevelViewport.ViewportConfig_OnePane", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts1Pane", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_OnePane.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts1Pane", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_TwoPanesH", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts2Panes", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_TwoPanesH.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts2Panes", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_TwoPanesV", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts2PanesStacked", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_TwoPanesV.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts2PanesStacked", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_ThreePanesLeft", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesLeft", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_ThreePanesLeft.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesLeft", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_ThreePanesRight", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesRight", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_ThreePanesRight.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesRight", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_ThreePanesTop", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesTop", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_ThreePanesTop.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesTop", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_ThreePanesBottom", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesBottom", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_ThreePanesBottom.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts3PanesBottom", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_FourPanesLeft", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesLeft", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_FourPanesLeft.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesLeft", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_FourPanesRight", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesRight", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_FourPanesRight.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesRight", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_FourPanesTop", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesTop", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_FourPanesTop.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesTop", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_FourPanesBottom", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesBottom", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_FourPanesBottom.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4PanesBottom", IconLayoutSizeSmall));
		Set("LevelViewport.ViewportConfig_FourPanes2x2", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4Panes", IconLayoutSize));
		Set("LevelViewport.ViewportConfig_FourPanes2x2.Small", new IMAGE_BRUSH_SVG("Starship/ViewportLayout/Layouts4Panes", IconLayoutSizeSmall));

		Set("LevelViewport.UseDefaultShowFlags", new IMAGE_BRUSH_SVG("Starship/Common/ResetToDefault", Icon16x16));
		
		Set( "LevelViewport.EjectActorPilot", new IMAGE_BRUSH_SVG( "Starship/Common/StopPiloting_16", Icon16x16 ) );
		Set( "LevelViewport.EjectActorPilot.Small", new IMAGE_BRUSH_SVG( "Starship/Common/StopPiloting_16", Icon16x16 ) );
		Set( "LevelViewport.PilotSelectedActor", new IMAGE_BRUSH_SVG( "Starship/EditorViewport/pilot", Icon16x16 ) );
		Set( "LevelViewport.SelectPilotedActor", new IMAGE_BRUSH_SVG( "Starship/EditorViewport/pilot-select", Icon16x16 ));

		Set( "LevelViewport.ToggleActorPilotCameraView",       new IMAGE_BRUSH_SVG(  "Starship/Common/ExactCameraView_16", Icon16x16 ) );
		Set( "LevelViewport.ToggleActorPilotCameraView.Small", new IMAGE_BRUSH_SVG( "Starship/Common/ExactCameraView_16", Icon16x16 ) );
	}

	// Level editor status bar
	{
		Set( "TransformSettings.RelativeCoordinateSettings", new IMAGE_BRUSH( "Icons/icon_axis_16px", Icon16x16 ) );
	}

	// Mesh Proxy Window
	{
		Set("MeshProxy.SimplygonLogo", new IMAGE_BRUSH( "Icons/SimplygonBanner_Sml", FVector2f(174.f, 36.f) ) );
	}
#endif // WITH_EDITOR

	// Level viewport 
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	{

		Set( "LevelViewport.DebugBorder", new BOX_BRUSH( "Old/Window/ViewportDebugBorder", 0.8f, FLinearColor(.7f,0.f,0.f,.5f) ) );
		Set( "LevelViewport.BlackBackground", new FSlateColorBrush( FLinearColor::Red ) ); 
		Set( "LevelViewport.StartingPlayInEditorBorder", new BOX_BRUSH( "Old/Window/ViewportDebugBorder", 0.8f, FLinearColor(0.1f,1.0f,0.1f,1.0f) ) );
		Set( "LevelViewport.StartingSimulateBorder", new BOX_BRUSH( "Old/Window/ViewportDebugBorder", 0.8f, FLinearColor(1.0f,1.0f,0.1f,1.0f) ) );
		Set( "LevelViewport.NonMaximizedBorder", new CORE_BORDER_BRUSH("Common/PlainBorder", 2.f / 8.f, FStyleColors::Black));
		Set( "LevelViewport.ReturningToEditorBorder", new BOX_BRUSH( "Old/Window/ViewportDebugBorder", 0.8f, FLinearColor(0.1f,0.1f,1.0f,1.0f) ) );
		Set( "LevelViewport.ActorLockIcon", new IMAGE_BRUSH( "Icons/ActorLockedViewport", Icon32x32 ) );
		Set( "LevelViewport.Icon", new IMAGE_BRUSH( "Icons/icon_tab_viewport_16px", Icon16x16 ) );

		Set( "LevelViewportContextMenu.ActorType.Text", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 7 ) )
			.SetColorAndOpacity( FSlateColor::UseSubduedForeground() ) );

		Set( "LevelViewportContextMenu.AssetLabel.Text", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 9 ) )
			.SetColorAndOpacity( FSlateColor::UseForeground() ) );

		Set( "LevelViewportContextMenu.AssetTileItem.ThumbnailAreaBackground", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f) );
		
		FLinearColor TransparentRecessed = FStyleColors::Recessed.GetSpecifiedColor();
		TransparentRecessed.A = 0.3f;
		Set( "LevelViewportContextMenu.AssetTileItem.NameAreaBackground", new FSlateRoundedBoxBrush(TransparentRecessed, 4.0f) );

		Set( "LevelViewport.CursorIcon", new IMAGE_BRUSH( "Common/Cursor", Icon16x16 ) );
	}

	// Show flags menus
	{
		Set( "ShowFlagsMenu.AntiAliasing", new IMAGE_BRUSH_SVG( "Starship/Common/AntiAliasing", Icon16x16 ) );
		Set( "ShowFlagsMenu.Atmosphere", new IMAGE_BRUSH_SVG( "Starship/Common/Atmosphere", Icon16x16 ) );
		Set( "ShowFlagsMenu.Cloud", new IMAGE_BRUSH_SVG( "Starship/AssetIcons/VolumetricCloud_16", Icon16x16 ) );
		Set( "ShowFlagsMenu.BSP", new IMAGE_BRUSH_SVG( "Starship/Common/BSP", Icon16x16 ) );
		Set( "ShowFlagsMenu.Collision", new IMAGE_BRUSH_SVG( "Starship/Common/Collision", Icon16x16 ) );
		Set( "ShowFlagsMenu.Decals", new IMAGE_BRUSH_SVG( "Starship/Common/Decals", Icon16x16 ) );
		Set( "ShowFlagsMenu.Fog", new IMAGE_BRUSH_SVG( "Starship/Common/Fog", Icon16x16 ) );
		Set( "ShowFlagsMenu.Grid", new IMAGE_BRUSH_SVG( "Starship/Common/Grid", Icon16x16 ) );
		Set( "ShowFlagsMenu.Landscape", new IMAGE_BRUSH_SVG( "Starship/Common/Landscape", Icon16x16 ) );
		Set( "ShowFlagsMenu.MediaPlanes", new IMAGE_BRUSH_SVG( "Starship/Common/MediaPlanes", Icon16x16 ) );
		Set( "ShowFlagsMenu.Navigation", new IMAGE_BRUSH_SVG( "Starship/Common/Navigation", Icon16x16 ) );
		Set( "ShowFlagsMenu.Particles", new IMAGE_BRUSH_SVG( "Starship/Common/ParticleSprites", Icon16x16 ) );
		Set( "ShowFlagsMenu.SkeletalMeshes", new IMAGE_BRUSH_SVG( "Starship/Common/SkeletalMesh", Icon16x16 ) );
		Set( "ShowFlagsMenu.StaticMeshes", new IMAGE_BRUSH_SVG( "Starship/Common/StaticMesh", Icon16x16 ) );
		Set( "ShowFlagsMenu.Translucency", new IMAGE_BRUSH_SVG( "Starship/Common/Transparency", Icon16x16 ) );
		Set( "ShowFlagsMenu.WidgetComponents", new IMAGE_BRUSH_SVG( "Starship/Common/WidgetComponents", Icon16x16 ) );
		Set( "ShowFlagsMenu.Cameras", new IMAGE_BRUSH_SVG( "Starship/AssetIcons/Camera_16", Icon16x16 ) );
		Set( "ShowFlagsMenu.Hair", new IMAGE_BRUSH( "Common/icon_ShowHair_16x", Icon16x16 ) );

		Set("ShowFlagsMenu.SubMenu.PostProcessing", new IMAGE_BRUSH_SVG("Starship/Common/PostProcessing", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.LightTypes", new IMAGE_BRUSH_SVG("Starship/Common/LightTypes", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.LightingComponents", new IMAGE_BRUSH_SVG("Starship/Common/LightingComponents", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.LightingFeatures", new IMAGE_BRUSH_SVG("Starship/Common/LightingFeatures", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.Lumen", new IMAGE_BRUSH_SVG("Starship/Common/LightingFeatures", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.MegaLights", new IMAGE_BRUSH_SVG("Starship/Common/LightingFeatures", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.Nanite", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Advanced", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.Developer", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Developer", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.Visualize", new IMAGE_BRUSH_SVG("Starship/Common/Visualize", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.Advanced", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Advanced", Icon16x16));

		Set("ShowFlagsMenu.SubMenu.Volumes", new IMAGE_BRUSH_SVG("Starship/Common/Volume", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.Layers", new IMAGE_BRUSH_SVG("Starship/Common/Layers", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.FoliageTypes", new IMAGE_BRUSH_SVG("Starship/Common/FoliageTypes", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.Sprites", new IMAGE_BRUSH_SVG("Starship/Common/Sprite", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.HLODs", new IMAGE_BRUSH_SVG("Starship/Common/HierarchicalLOD", Icon16x16));
		Set("ShowFlagsMenu.SubMenu.RevisionControl", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/Status/RevisionControl", Icon16x16));
	}
	

#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

	// Mobility Icons
	{
		Set("Mobility.Movable", new IMAGE_BRUSH("/Icons/Mobility/Movable_16x", Icon16x16));
		Set("Mobility.Stationary", new IMAGE_BRUSH("/Icons/Mobility/Adjustable_16x", Icon16x16));
		Set("Mobility.Static", new IMAGE_BRUSH("/Icons/Mobility/Static_16x", Icon16x16));

		const FString SmallRoundedButton(TEXT("Common/SmallRoundedToggle"));
		const FString SmallRoundedButtonStart(TEXT("Common/SmallRoundedToggleLeft"));
		const FString SmallRoundedButtonMiddle(TEXT("Common/SmallRoundedToggleCenter"));
		const FString SmallRoundedButtonEnd(TEXT("Common/SmallRoundedToggleRight"));

		const FLinearColor NormalColor(0.15f, 0.15f, 0.15f, 1.f);

		Set("Property.ToggleButton", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), NormalColor))
			.SetUncheckedPressedImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), SelectionColor_Pressed))
			.SetUncheckedHoveredImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), SelectionColor_Pressed))
			.SetCheckedHoveredImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), SelectionColor))
			.SetCheckedPressedImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), SelectionColor))
			.SetCheckedImage(BOX_BRUSH(*SmallRoundedButton, FMargin(7.f / 16.f), SelectionColor)));

		Set("Property.ToggleButton.Start", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), NormalColor))
			.SetUncheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), SelectionColor_Pressed))
			.SetUncheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), SelectionColor_Pressed))
			.SetCheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), SelectionColor))
			.SetCheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), SelectionColor))
			.SetCheckedImage(BOX_BRUSH(*SmallRoundedButtonStart, FMargin(7.f / 16.f), SelectionColor)));

		Set("Property.ToggleButton.Middle", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), NormalColor))
			.SetUncheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), SelectionColor_Pressed))
			.SetUncheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), SelectionColor_Pressed))
			.SetCheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), SelectionColor))
			.SetCheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), SelectionColor))
			.SetCheckedImage(BOX_BRUSH(*SmallRoundedButtonMiddle, FMargin(7.f / 16.f), SelectionColor)));

		Set("Property.ToggleButton.End", FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), NormalColor))
			.SetUncheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), SelectionColor_Pressed))
			.SetUncheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), SelectionColor_Pressed))
			.SetCheckedHoveredImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), SelectionColor))
			.SetCheckedPressedImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), SelectionColor))
			.SetCheckedImage(BOX_BRUSH(*SmallRoundedButtonEnd, FMargin(7.f / 16.f), SelectionColor)));

		// Experimental/early access stuff
		Set("PropertyEditor.ExperimentalClass", new IMAGE_BRUSH("/PropertyView/ExperimentalClassWarning", Icon40x40));
		Set("PropertyEditor.EarlyAccessClass", new IMAGE_BRUSH("/PropertyView/EarlyAccessClassWarning", Icon40x40));
	}

	// Mesh Paint
	{
		Set("MeshPaint.SelectVertex", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Select_40x"), Icon20x20));
		Set("MeshPaint.SelectVertex.Small", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Select_40x"), Icon20x20));
		Set("MeshPaint.SelectTextureColor", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Select_40x"), Icon20x20));
		Set("MeshPaint.SelectTextureColor.Small", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Select_40x"), Icon20x20));
		Set("MeshPaint.SelectTextureAsset", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Select_40x"), Icon20x20));
		Set("MeshPaint.SelectTextureAsset.Small", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Select_40x"), Icon20x20));
		Set("MeshPaint.PaintVertexColor", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("MeshPaint.PaintVertexColor.Small", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("MeshPaint.PaintVertexWeight", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("MeshPaint.PaintVertexWeight.Small", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("MeshPaint.PaintTextureColor", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("MeshPaint.PaintTextureColor.Small", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("MeshPaint.PaintTextureAsset", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("MeshPaint.PaintTextureAsset.Small", new IMAGE_BRUSH("Icons/GeneralTools/Paint_40x", Icon20x20));
		Set("MeshPaint.SwapColor", new IMAGE_BRUSH("Icons/Paint/Paint_SwapColors_40x", Icon20x20));
		Set("MeshPaint.SwapColor.Small", new IMAGE_BRUSH("Icons/Paint/Paint_SwapColors_40x", Icon20x20));
		Set("MeshPaint.FillVertex", new IMAGE_BRUSH("/Icons/GeneralTools/PaintBucket_40x", Icon20x20));
		Set("MeshPaint.FillVertex.Small", new IMAGE_BRUSH("/Icons/GeneralTools/PaintBucket_40x", Icon20x20));
		Set("MeshPaint.FillTexture", new IMAGE_BRUSH("/Icons/GeneralTools/PaintBucket_40x", Icon20x20));
		Set("MeshPaint.FillTexture.Small", new IMAGE_BRUSH("/Icons/GeneralTools/PaintBucket_40x", Icon20x20));
		Set("MeshPaint.PropagateMesh", new IMAGE_BRUSH("/Icons/Paint/Paint_Propagate_40x", Icon20x20));
		Set("MeshPaint.PropagateMesh.Small", new IMAGE_BRUSH("/Icons/Paint/Paint_Propagate_40x", Icon20x20));
		Set("MeshPaint.PropagateLODs", new IMAGE_BRUSH("Icons/Paint/Paint_AllLODs_40x", Icon20x20));
		Set("MeshPaint.PropagateLODs.Small", new IMAGE_BRUSH("Icons/Paint/Paint_AllLODs_40x", Icon20x20));
		Set("MeshPaint.SaveVertex", new IMAGE_BRUSH("/Icons/GeneralTools/Save_40x", Icon20x20));
		Set("MeshPaint.SaveVertex.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Save_40x", Icon20x20));
		Set("MeshPaint.SaveTexture", new IMAGE_BRUSH("/Icons/GeneralTools/Save_40x", Icon20x20));
		Set("MeshPaint.SaveTexture.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Save_40x", Icon20x20));
		Set("MeshPaint.Add", new IMAGE_BRUSH("/Icons/icon_add_40x", Icon20x20));
		Set("MeshPaint.Add.Small", new IMAGE_BRUSH("/Icons/icon_add_40x", Icon20x20));
		Set("MeshPaint.RemoveVertex", new IMAGE_BRUSH("/Icons/GeneralTools/Delete_40x", Icon20x20));
		Set("MeshPaint.RemoveVertex.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Delete_40x", Icon20x20));
		Set("MeshPaint.RemoveTexture", new IMAGE_BRUSH("/Icons/GeneralTools/Delete_40x", Icon20x20));
		Set("MeshPaint.RemoveTexture.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Delete_40x", Icon20x20));
		Set("MeshPaint.Copy", new IMAGE_BRUSH("/Icons/GeneralTools/Copy_40x", Icon20x20));
		Set("MeshPaint.Copy.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Copy_40x", Icon20x20));
		Set("MeshPaint.Paste", new IMAGE_BRUSH("/Icons/GeneralTools/Paste_40x", Icon20x20));
		Set("MeshPaint.Paste.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Paste_40x", Icon20x20));
		Set("MeshPaint.Import", new IMAGE_BRUSH("/Icons/GeneralTools/Import_40x", Icon20x20));
		Set("MeshPaint.Import.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Import_40x", Icon20x20));
		Set("MeshPaint.GetTextureColors", new IMAGE_BRUSH("/Icons/GeneralTools/Import_40x", Icon20x20));
		Set("MeshPaint.GetTextureColors.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Import_40x", Icon20x20));
		Set("MeshPaint.GetVertexColors", new IMAGE_BRUSH("/Icons/GeneralTools/Import_40x", Icon20x20));
		Set("MeshPaint.GetVertexColors.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Import_40x", Icon20x20));
		Set("MeshPaint.FixVertex", new IMAGE_BRUSH("/Icons/GeneralTools/Fix_40x", Icon20x20));
		Set("MeshPaint.FixVertex.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Fix_40x", Icon20x20));
		Set("MeshPaint.FixTexture", new IMAGE_BRUSH("/Icons/GeneralTools/Fix_40x", Icon20x20));
		Set("MeshPaint.FixTexture.Small", new IMAGE_BRUSH("/Icons/GeneralTools/Fix_40x", Icon20x20));
		Set("MeshPaint.PreviousLOD", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Previous_40x"), Icon20x20));
		Set("MeshPaint.PreviousLOD.Small", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Previous_40x"), Icon20x20));
		Set("MeshPaint.NextLOD", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Next_40x"), Icon20x20));
		Set("MeshPaint.NextLOD.Small", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Next_40x"), Icon20x20));
		Set("MeshPaint.PreviousTexture", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Previous_40x"), Icon20x20));
		Set("MeshPaint.PreviousTexture.Small", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Previous_40x"), Icon20x20));
		Set("MeshPaint.NextTexture", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Next_40x"), Icon20x20));
		Set("MeshPaint.NextTexture.Small", new IMAGE_BRUSH(TEXT("Icons/GeneralTools/Next_40x"), Icon20x20));
		Set("MeshPaint.Brush", new IMAGE_BRUSH_SVG("Starship/Common/Paintbrush", Icon20x20));
		Set("MeshPaint.FindInCB", new IMAGE_BRUSH("/Icons/icon_toolbar_genericfinder_40px", Icon20x20));
		Set("MeshPaint.Swap", new IMAGE_BRUSH("/Icons/icon_MeshPaint_Swap_16x", Icon12x12));
	}

	// Scalability (Performance Warning)
	{
		Set( "Scalability.ScalabilitySettings", new IMAGE_BRUSH("Scalability/ScalabilitySettings", FVector2f(473.0f, 266.0f) ) );
	}

	Set("WorkspaceMenu.AdditionalUI", new IMAGE_BRUSH("Icons/icon_ViewMode_LODColoration_16px", Icon16x16));
}

void FStarshipEditorStyle::FStyle::SetupPersonaStyle()
{
	// Persona
#if WITH_EDITOR
	{
		// Persona viewport
		Set( "AnimViewportMenu.TranslateMode", new IMAGE_BRUSH( "Icons/icon_translate_40x", Icon32x32) );
		Set( "AnimViewportMenu.TranslateMode.Small", new IMAGE_BRUSH( "Icons/icon_translate_40x", Icon16x16 ) );
		Set( "AnimViewportMenu.RotateMode", new IMAGE_BRUSH( "Icons/icon_rotate_40x", Icon32x32) );
		Set( "AnimViewportMenu.RotateMode.Small", new IMAGE_BRUSH( "Icons/icon_rotate_40x", Icon16x16 ) );
		Set( "AnimViewportMenu.CameraFollow", new IMAGE_BRUSH( "Persona/Viewport/Camera_FollowBounds_40px", Icon32x32) );
		Set( "AnimViewportMenu.CameraFollow.Small", new IMAGE_BRUSH( "Persona/Viewport/Camera_FollowBounds_40px", Icon16x16 ) );
		Set( "AnimViewport.LocalSpaceEditing", new IMAGE_BRUSH( "Icons/icon_axis_local_16px", Icon16x16 ) );
		Set( "AnimViewport.WorldSpaceEditing", new IMAGE_BRUSH( "Icons/icon_axis_world_16px", Icon16x16 ) );
		Set( "AnimViewportMenu.SetShowNormals", new IMAGE_BRUSH( TEXT("Icons/icon_StaticMeshEd_Normals_40x"), Icon40x40 ) );
		Set( "AnimViewportMenu.SetShowNormals.Small", new IMAGE_BRUSH( TEXT("Icons/icon_StaticMeshEd_Normals_40x"), Icon20x20 ) );
		Set( "AnimViewportMenu.SetShowTangents", new IMAGE_BRUSH( TEXT("Icons/icon_StaticMeshEd_Tangents_40x"), Icon40x40 ) );
		Set( "AnimViewportMenu.SetShowTangents.Small", new IMAGE_BRUSH( TEXT("Icons/icon_StaticMeshEd_Tangents_40x"), Icon20x20 ) );
		Set( "AnimViewportMenu.SetShowBinormals", new IMAGE_BRUSH( TEXT("Icons/icon_StaticMeshEd_Binormals_40x"), Icon40x40 ) );
		Set( "AnimViewportMenu.SetShowBinormals.Small", new IMAGE_BRUSH( TEXT("Icons/icon_StaticMeshEd_Binormals_40x"), Icon20x20 ) );
		Set( "AnimViewportMenu.AnimSetDrawUVs", new IMAGE_BRUSH( TEXT("Icons/icon_StaticMeshEd_UVOverlay_40x"), Icon40x40 ) );
		Set( "AnimViewportMenu.AnimSetDrawUVs.Small", new IMAGE_BRUSH( TEXT("Icons/icon_StaticMeshEd_UVOverlay_40x"), Icon20x20 ) );

		Set( "AnimViewportMenu.SkinWeightProfiles", new IMAGE_BRUSH_SVG( TEXT("Starship/Persona/SkinWeightProfiles"), Icon16x16 ) );

		Set("AnimViewportMenu.PlayBackSpeed", new IMAGE_BRUSH_SVG("Starship/Common/play", Icon16x16));
		Set("AnimViewportMenu.TurnTableSpeed", new IMAGE_BRUSH("Persona/Viewport/icon_turn_table_16x", Icon16x16));
		Set("AnimViewportMenu.SceneSetup", new IMAGE_BRUSH("Icons/icon_tab_SceneOutliner_16x", Icon16x16));

		Set( "AnimViewport.MessageFont", DEFAULT_FONT("Bold", 9) );

		Set("AnimViewport.MessageText", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f)));
		Set("AnimViewport.WarningText", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize))
			.SetColorAndOpacity(FLinearColor::Yellow)
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f)));
		Set("AnimViewport.ErrorText", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize))
			.SetColorAndOpacity(FLinearColor::Red)
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f)));
		
		// Viewport notifications 
		Set("AnimViewport.Notification.Error", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.728f, 0.0f, 0.0f)));
		Set("AnimViewport.Notification.Warning", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.728f, 0.364f, 0.003f)));
		Set("AnimViewport.Notification.Message", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FLinearColor(0.364f, 0.364f, 0.364f)));

		Set("AnimViewport.Notification.CloseButton", FButtonStyle()
			.SetNormal(IMAGE_BRUSH("/Docking/CloseApp_Normal", Icon16x16))
			.SetPressed(IMAGE_BRUSH("/Docking/CloseApp_Pressed", Icon16x16))
			.SetHovered(IMAGE_BRUSH("/Docking/CloseApp_Hovered", Icon16x16)));

		// persona commands
		Set("Persona.AnimNotifyWindow", new IMAGE_BRUSH_SVG("Starship/Persona/AnimationNotifies", Icon20x20));
		Set("Persona.RetargetManager", new IMAGE_BRUSH_SVG("Starship/Persona/RetargetManager", Icon20x20));
		Set("Persona.ImportMesh", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import", Icon20x20));
		Set("Persona.ReimportMesh", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));
		Set("Persona.ImportLODs", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import", Icon20x20));
		Set("Persona.ImportAnimation", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import", Icon20x20));
		Set("Persona.ReimportAnimation", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));
		Set("Persona.ApplyCompression", new IMAGE_BRUSH_SVG("Starship/Common/Compress", Icon20x20));
		Set("Persona.ExportToFBX", new CORE_IMAGE_BRUSH_SVG("Starship/Common/export_20", Icon20x20));
		Set("Persona.CreateAsset", new IMAGE_BRUSH_SVG("Starship/Persona/PersonaCreateAsset", Icon20x20));
		Set("Persona.StartRecordAnimation", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_StartRecord_40x"), Icon40x40));
		Set("Persona.StopRecordAnimation", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_StopRecord_40x"), Icon40x40));
		Set("Persona.StopRecordAnimation_Alt", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_StopRecord_Alt_40x"), Icon40x40));
		Set("Persona.SetKey", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_SetKey_40x"), Icon40x40));
		Set("Persona.ApplyAnimation", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_BakeAnim_40x"), Icon40x40));
		Set("Persona.EditInSequencer", new IMAGE_BRUSH_SVG("Starship/Persona/EditInSequencer", Icon20x20));

		// preview set up
		Set("Persona.TogglePreviewAsset", new IMAGE_BRUSH_SVG("Starship/Persona/AnimationPreviewMesh", Icon20x20));
		Set("Persona.TogglePreviewAnimation", new IMAGE_BRUSH_SVG("Starship/Persona/PersonaPreviewAnimation", Icon20x20));
		Set("Persona.ToggleReferencePose", new IMAGE_BRUSH_SVG("Starship/Persona/PersonaTPose", Icon20x20));
		Set("Persona.SavePreviewMeshCollection", new IMAGE_BRUSH(TEXT("Icons/Save_16x"), Icon16x16));

		// persona extras
		Set("Persona.ConvertAnimationGraph", new IMAGE_BRUSH("Old/Graph/ConvertIcon", Icon40x40));
		Set("Persona.ReimportAsset", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));
		Set("Persona.ConvertToStaticMesh", new IMAGE_BRUSH_SVG("Starship/Common/MakeStaticMesh", Icon20x20));
		Set("Persona.BakeMaterials", new IMAGE_BRUSH("Icons/icon_tab_Layers_40x", Icon40x40));

		// Anim Slot Manager
		Set("AnimSlotManager.SaveSkeleton", new IMAGE_BRUSH("Persona/AnimSlotManager/icon_SaveSkeleton_40x", Icon40x40));
		Set("AnimSlotManager.AddGroup", new IMAGE_BRUSH("Persona/AnimSlotManager/icon_AddGroup_40x", Icon40x40));
		Set("AnimSlotManager.AddSlot", new IMAGE_BRUSH("Persona/AnimSlotManager/icon_AddSlot_40x", Icon40x40));
		Set("AnimSlotManager.Warning", new IMAGE_BRUSH("Persona/AnimSlotManager/icon_Warning_14x", Icon16x16));

		// Anim Notify Editor
		Set("AnimNotifyEditor.BranchingPoint", new IMAGE_BRUSH("Persona/NotifyEditor/BranchingPoints_24x", Icon24x24));
		Set("AnimNotifyEditor.AnimNotify", new IMAGE_BRUSH_SVG("Starship/Persona/Notify", Icon16x16));
		Set("AnimNotifyEditor.AnimSyncMarker", new IMAGE_BRUSH_SVG("Starship/Persona/SyncMarker", Icon16x16));

		// AnimBlueprint Preview Warning Background
		FSlateColor PreviewPropertiesWarningColour(FLinearColor::Gray);
		Set("Persona.PreviewPropertiesWarning", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, PreviewPropertiesWarningColour));

		// Persona-specific tabs
		Set("Persona.Tabs.SkeletonTree", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_Skeleton_Tree_16x"), Icon16x16));
		Set("Persona.Tabs.MorphTargetPreviewer", new IMAGE_BRUSH_SVG(TEXT("Starship/Persona/MorphTarget"), Icon16x16));
		Set("Persona.Tabs.AnimCurveDebugger", new IMAGE_BRUSH_SVG(TEXT("Starship/Persona/CurveDebugger"), Icon16x16));
		Set("Persona.Tabs.AnimCurveMetadataEditor", new IMAGE_BRUSH_SVG(TEXT("Starship/Persona/CurveMetadata"), Icon16x16));
		Set("Persona.Tabs.AnimationNotifies", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_Animation_Notifies_16x"), Icon16x16));
		Set("Persona.Tabs.RetargetManager", new IMAGE_BRUSH_SVG("Starship/Persona/RetargetManager", Icon16x16));
		Set("Persona.Tabs.AnimSlotManager", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_Anim_Slot_Manager_16x"), Icon16x16));
		Set("Persona.Tabs.SkeletonCurves", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_Skeleton_Curves_16x"), Icon16x16));
		Set("Persona.Tabs.AnimAssetDetails", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_Anim_Asset_Details_16x"), Icon16x16));
		Set("Persona.Tabs.ControlRigMappingWindow", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_Skeleton_Tree_16x"), Icon16x16));
		Set("Persona.Tabs.FindReplace", new IMAGE_BRUSH_SVG(TEXT("Starship/Persona/FindReplace_16"), Icon16x16));

		// AssetFamilyIcons
		Set("Persona.AssetClass.Skeleton", new IMAGE_BRUSH_SVG("Starship/Persona/Skeleton", Icon20x20));
		Set("Persona.AssetClass.Animation", new IMAGE_BRUSH_SVG("Starship/Common/Animation", Icon20x20));
		Set("Persona.AssetClass.SkeletalMesh", new IMAGE_BRUSH_SVG("Starship/Persona/SkeletalMesh", Icon20x20));
		Set("Persona.AssetClass.Blueprint", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/blueprints", Icon20x20) );
		Set("Persona.AssetClass.Physics", new IMAGE_BRUSH_SVG("Starship/Persona/Physics", Icon20x20));

		// Find/replace tab
		Set("Persona.FindReplace.MatchCase", new IMAGE_BRUSH_SVG("Starship/Persona/MatchCase_20", Icon20x20));
		Set("Persona.FindReplace.MatchWholeWord", new IMAGE_BRUSH_SVG("Starship/Persona/MatchWord_20", Icon20x20));
	}

	// Skeleton editor
	{
		Set("SkeletonEditor.AnimNotifyWindow", new IMAGE_BRUSH_SVG("Starship/Persona/AnimationNotifies", Icon20x20));
		Set("SkeletonEditor.RetargetManager", new IMAGE_BRUSH_SVG("Starship/Persona/RetargetManager", Icon20x20));
		Set("SkeletonEditor.ImportMesh", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import", Icon20x20));

		// Skeleton Tree
		Set("SkeletonTree.SkeletonSocket", new IMAGE_BRUSH("Persona/SkeletonTree/icon_SocketG_16px", Icon16x16));
		Set("SkeletonTree.MeshSocket", new IMAGE_BRUSH("Persona/SkeletonTree/icon_SocketC_16px", Icon16x16));
		Set("SkeletonTree.LODBone", new IMAGE_BRUSH(TEXT("Persona/SkeletonTree/icon_LODBone_16x"), Icon16x16));
		Set("SkeletonTree.Bone", 	new IMAGE_BRUSH_SVG(TEXT("Starship/Animation/Bone"), Icon16x16) );
		Set("SkeletonTree.BoneNonWeighted", new IMAGE_BRUSH_SVG(TEXT("Starship/Animation/BoneNonWeighted"), Icon16x16) );
		Set("SkeletonTree.NonRequiredBone", new IMAGE_BRUSH(TEXT("Persona/SkeletonTree/icon_NonRequiredBone_16x"), Icon16x16));
		Set("SkeletonTree.NormalFont", FTextBlockStyle(NormalText));
		Set("SkeletonTree.ItalicFont", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Italic", 10)));

		Set("SkeletonTree.HyperlinkSpinBox", FSpinBoxStyle()
			.SetBackgroundBrush(       FSlateRoundedBoxBrush(FStyleColors::Input, 3.f, FStyleColors::Transparent, 1.0f))
			.SetHoveredBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, 3.f, FStyleColors::Transparent, 1.0f))
			.SetActiveFillBrush(       FSlateRoundedBoxBrush(FStyleColors::Hover, 3.f, FStyleColors::Transparent, 1.0f))
			.SetInactiveFillBrush(     FSlateRoundedBoxBrush(FStyleColors::Secondary, 3.f, FStyleColors::Transparent, 1.0f))
			.SetArrowsImage(FSlateNoResource())
			.SetForegroundColor(FStyleColors::ForegroundHover)
			.SetTextPadding(FMargin(8.f, 4.f, 8.f, 3.f))
			.SetArrowsImage(FSlateNoResource())
		);


		const FButtonStyle BorderlessButton = FButtonStyle(GetWidgetStyle<FButtonStyle>("SimpleButton"))
			.SetNormalPadding(0.f)
			.SetPressedPadding(0.f);

		Set("SkeletonTree.RetargetingComboButton", FComboButtonStyle(GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.SetButtonStyle(BorderlessButton)
			.SetDownArrowPadding(FMargin(2.0f, 0.0f, 0.0f, 0.0f)));

		Set("SkeletonTree.BlendProfile", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_NewBlendSpace_16x"), Icon16x16));
		Set("SkeletonTree.InlineEditorShadowTop", new IMAGE_BRUSH(TEXT("Common/ScrollBoxShadowTop"), FVector2f(64.f, 8.f)));
		Set("SkeletonTree.InlineEditorShadowBottom", new IMAGE_BRUSH(TEXT("Common/ScrollBoxShadowBottom"), FVector2f(64.f, 8.f)));
	}

	// Animation editor
	{
		Set("AnimationEditor.ApplyCompression", new IMAGE_BRUSH_SVG("Starship/Common/Compress", Icon20x20));
		Set("AnimationEditor.ExportToFBX", new CORE_IMAGE_BRUSH_SVG("Starship/Common/export_20", Icon20x20));
		Set("AnimationEditor.ReimportAnimation", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));
		Set("AnimationEditor.ReimportAnimationWithDialog", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));
		Set("AnimationEditor.CreateAsset", new IMAGE_BRUSH_SVG("Starship/Persona/PersonaCreateAsset", Icon20x20));
		Set("AnimationEditor.SetKey", new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus", Icon20x20));
		Set("AnimationEditor.ApplyAnimation", new IMAGE_BRUSH_SVG("Starship/Common/Apply", Icon20x20));

		Set("AnimTimeline.Outliner.DefaultBorder", new FSlateColorBrush(FLinearColor::White));

		const FSplitterStyle OutlinerSplitterStyle = FSplitterStyle()
			.SetHandleNormalBrush(FSlateColorBrush(FStyleColors::Recessed))
			.SetHandleHighlightBrush(FSlateColorBrush(FStyleColors::Secondary));

		Set("AnimTimeline.Outliner.Splitter", OutlinerSplitterStyle);

		Set("AnimTimeline.Outliner.Label", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2f::UnitVector));

		Set("AnimTimeline.Outliner.ItemColor", FLinearColor(0.04f, 0.04f, 0.04f, 0.8f));
		Set("AnimTimeline.Outliner.HeaderColor", FLinearColor(0.03f, 0.03f, 0.03f, 1.0f));

		Set("AnimTimeline.SectionMarker", new IMAGE_BRUSH(TEXT("Sequencer/SectionMarker"), FVector2f(11.f, 12.f)));
	}

	// Skeletal mesh editor
	{
		Set("SkeletalMeshEditor.ReimportMesh", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));
		Set("SkeletalMeshEditor.ReimportWithDialog", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));
		Set("SkeletalMeshEditor.ImportLODs", new CORE_IMAGE_BRUSH_SVG("Starship/Common/import", Icon20x20));

		Set("SkeletalMeshEditor.MeshSectionSelection", new IMAGE_BRUSH_SVG("Starship/Persona/SectionSelection", Icon20x20));
	}

	// Motion Matching editor
	{
		Set("MotionMatchingEditor.EnablePoseReselection", new IMAGE_BRUSH_SVG("Starship/Animation/PoseReselection", Icon20x20));
		Set("MotionMatchingEditor.DisablePoseReselection", new IMAGE_BRUSH_SVG("Starship/Animation/PoseReselection", Icon20x20, FLinearColor(1.0f, 1.0f, 1.0f, 0.25f)));
	}
	
	// New anim blueprint dialog
	{
		Set("NewAnimBlueprintDialog.AreaBorder", new FSlateRoundedBoxBrush(FStyleColors::Panel, 4.0f));

		const FCheckBoxStyle CheckBoxAreaStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Secondary, 4.0f))
			.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f))
			.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::PrimaryPress, 4.0f))
			.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::Primary, 1.0f))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::PrimaryHover, 1.0f))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Transparent, 4.0f, FStyleColors::PrimaryPress, 1.0f));

		Set("NewAnimBlueprintDialog.CheckBoxArea", CheckBoxAreaStyle);
	}

	// Kismet 2
	{
		Set( "FullBlueprintEditor.SwitchToScriptingMode", new IMAGE_BRUSH_SVG( "Starship/Blueprints/icon_BlueprintEditor_EventGraph", Icon20x20));

		// @todo - Icon Replacement - The UI commands using these icons are never visible in the editor
		Set( "FullBlueprintEditor.SwitchToBlueprintDefaultsMode", new IMAGE_BRUSH_SVG("Starship/Common/blueprint", Icon20x20));
		Set( "FullBlueprintEditor.SwitchToComponentsMode", new IMAGE_BRUSH_SVG("Starship/Common/Component", Icon20x20));

		Set( "FullBlueprintEditor.EditGlobalOptions", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/Settings", Icon20x20));
		Set("FullBlueprintEditor.EditClassDefaults", new IMAGE_BRUSH_SVG("Starship/Common/Details", Icon20x20));

		Set( "FullBlueprintEditor.Diff", new IMAGE_BRUSH( "Icons/BlueprintEditorDiff", Icon40x40 ) );
		Set( "FullBlueprintEditor.Diff.Small", new IMAGE_BRUSH( "Icons/BlueprintEditorDiff", Icon20x20 ) );

		Set( "BlueprintEditor.ActionMenu.ContextDescriptionFont",  DEFAULT_FONT("Regular", 12) );
		Set( "BlueprintEditor.ActionMenu.ContextDescriptionFont",  DEFAULT_FONT("Regular", 12) );

		Set("BlueprintEditor.FindInBlueprints.MenuIcon", new IMAGE_BRUSH_SVG("Starship/Common/FindInBlueprints", Icon16x16));
		Set("BlueprintEditor.FindInBlueprint", new IMAGE_BRUSH_SVG("Starship/Common/FindInBlueprints", Icon20x20));

		Set( "Kismet.DeleteUnusedVariables", new IMAGE_BRUSH_SVG("/Starship/Blueprints/icon_kismet_findunused", Icon16x16) );
		{
			Set( "Kismet.Tabs.Variables", new IMAGE_BRUSH_SVG( "Starship/Blueprints/pill", Icon16x16 ) );
			Set( "Kismet.Tabs.Palette", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Palette", Icon16x16 ) );
			Set( "Kismet.Tabs.CompilerResults", new CORE_IMAGE_BRUSH_SVG("Starship/Common/OutputLog", Icon16x16 ) );
			Set( "Kismet.Tabs.FindResults", new CORE_IMAGE_BRUSH_SVG("Starship/Common/search", Icon16x16 ) );
			Set( "Kismet.Tabs.Bookmarks", new IMAGE_BRUSH_SVG( "Starship/Common/Bookmarks", Icon16x16 ) );
			Set( "Kismet.Tabs.Components", new IMAGE_BRUSH_SVG( "Starship/Common/Component", Icon16x16 ) );
			Set( "Kismet.Tabs.BlueprintDefaults", new IMAGE_BRUSH( "Icons/icon_BlueprintEditor_Defaults_40x", Icon16x16 ) );
		}

		const FCheckBoxStyle KismetFavoriteToggleStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
			.SetUncheckedImage( IMAGE_BRUSH("Icons/EmptyStar_16x", Icon10x10, FLinearColor(0.8f, 0.8f, 0.8f, 1.f)) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH("Icons/EmptyStar_16x", Icon10x10, FLinearColor(2.5f, 2.5f, 2.5f, 1.f)) )
			.SetUncheckedPressedImage( IMAGE_BRUSH("Icons/EmptyStar_16x", Icon10x10, FLinearColor(0.8f, 0.8f, 0.8f, 1.f)) )
			.SetCheckedImage( IMAGE_BRUSH("Icons/Star_16x", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)) )
			.SetCheckedHoveredImage( IMAGE_BRUSH("Icons/Star_16x", Icon10x10, FLinearColor(0.4f, 0.4f, 0.4f, 1.f)) )
			.SetCheckedPressedImage( IMAGE_BRUSH("Icons/Star_16x", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)) );
		Set("Kismet.Palette.FavoriteToggleStyle", KismetFavoriteToggleStyle);

		Set( "Kismet.Tooltip.SubtextFont", DEFAULT_FONT("Regular", 8) );

		Set("Blueprint.CompileStatus.Background", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20x20));

		// @todo - Icon Replacement - trying out tinting compile backgrounds
/*
		Set("Blueprint.CompileStatus.Background.Unknown", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20x20, FStyleColors::AccentYellow));
		Set("Blueprint.CompileStatus.Background.Warning", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20x20, FStyleColors::Warning));
		Set("Blueprint.CompileStatus.Background.Good", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20x20, FStyleColors::AccentGreen));
		Set("Blueprint.CompileStatus.Background.Error", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Background", Icon20x20, FStyleColors::Error));*/

		Set("Blueprint.CompileStatus.Overlay.Unknown", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Unknown_Badge", Icon20x20, FStyleColors::AccentYellow));
		Set("Blueprint.CompileStatus.Overlay.Warning", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Warning_Badge", Icon20x20, FStyleColors::Warning));
		Set("Blueprint.CompileStatus.Overlay.Good", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Good_Badge", Icon20x20, FStyleColors::AccentGreen));
		Set("Blueprint.CompileStatus.Overlay.Error", new IMAGE_BRUSH_SVG("Starship/Blueprints/CompileStatus_Fail_Badge", Icon20x20, FStyleColors::Error));

		// @todo - Icon Replacement - these are hijacked by non-blueprint things. Keeping them around for now
		Set( "Kismet.Status.Unknown", new IMAGE_BRUSH( "Old/Kismet2/CompileStatus_Working", Icon40x40 ) );
		Set( "Kismet.Status.Error", new IMAGE_BRUSH( "Old/Kismet2/CompileStatus_Fail", Icon40x40 ) );
		Set( "Kismet.Status.Good", new IMAGE_BRUSH( "Old/Kismet2/CompileStatus_Good", Icon40x40 ) );
		Set( "Kismet.Status.Warning", new IMAGE_BRUSH( "Old/Kismet2/CompileStatus_Warning", Icon40x40 ) );

		Set("BlueprintEditor.AddNewVariable", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_AddVariable", Icon20x20));
		Set("BlueprintEditor.AddNewLocalVariable", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_AddVariable", Icon20x20));
		Set("BlueprintEditor.AddNewFunction", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_AddFunction", Icon20x20));
		Set("BlueprintEditor.AddNewMacroDeclaration", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_AddMacro", Icon20x20));


		Set( "BlueprintEditor.AddNewAnimationGraph", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-plus", Icon16x16));
		Set( "BlueprintEditor.AddNewEventGraph", new IMAGE_BRUSH_SVG( "Starship/Blueprints/icon_Blueprint_AddGraph", Icon20x20) );

		Set( "BlueprintEditor.AddNewDelegate", new IMAGE_BRUSH_SVG( "Starship/Blueprints/icon_Blueprint_AddDelegate", Icon20x20) );

		Set( "BlueprintEditor.AddNewAnimationLayer", new IMAGE_BRUSH_SVG( "Starship/Blueprints/icon_Blueprint_AddFunction", Icon20x20) );


		Set("Kismet.VariableList.TypeIcon", new IMAGE_BRUSH_SVG( "Starship/Blueprints/pill", Icon16x16));
		Set("Kismet.VariableList.ArrayTypeIcon", new IMAGE_BRUSH_SVG( "Starship/Blueprints/pillarray", Icon16x16));
		Set("Kismet.VariableList.SetTypeIcon", new IMAGE_BRUSH_SVG( "Starship/Blueprints/pillset", Icon16x16));
		Set("Kismet.VariableList.SetTypeIconLarge", new IMAGE_BRUSH_SVG( "Starship/Blueprints/pillset", Icon40x40));
		Set("Kismet.VariableList.MapValueTypeIcon", new IMAGE_BRUSH_SVG( "Starship/Blueprints/pillmapvalue", Icon16x16));
		Set("Kismet.VariableList.MapKeyTypeIcon", new IMAGE_BRUSH_SVG( "Starship/Blueprints/pillmapkey", Icon16x16));

		Set("Kismet.VariableList.PromotableTypeOuterIcon", new IMAGE_BRUSH_SVG( "Starship/Blueprints/promotable_type_outer_icon", Icon14x14));
		Set("Kismet.VariableList.PromotableTypeInnerIcon", new IMAGE_BRUSH_SVG( "Starship/Blueprints/promotable_type_inner_icon", Icon14x14));

		Set("Kismet.VariableList.ExposeForInstance", new CORE_IMAGE_BRUSH_SVG("Starship/Common/visible", Icon16x16));
		Set("Kismet.VariableList.HideForInstance", new CORE_IMAGE_BRUSH_SVG("Starship/Common/hidden", Icon16x16));

		Set("Kismet.VariableList.FieldNotify", new CORE_IMAGE_BRUSH_SVG("Starship/Common/fieldnotify_on", Icon16x16));
		Set("Kismet.VariableList.NotFieldNotify", new CORE_IMAGE_BRUSH_SVG("Starship/Common/fieldnotify_off", Icon16x16));

		Set( "Kismet.Explorer.Title", FTextBlockStyle(NormalText) .SetFont(DEFAULT_FONT( "BoldCondensedItalic", 11)));
		Set( "Kismet.Explorer.SearchDepthFont", DEFAULT_FONT( "Bold", 14) );

		Set( "Kismet.Interfaces.Title",  FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT("Bold", 11 ) ) );
		Set( "Kismet.Interfaces.Implement", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/arrow-left", Icon16x16) );
		Set( "Kismet.Interfaces.Remove", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/arrow-right", Icon16x16) );

		Set( "Kismet.TypePicker.CategoryFont", DEFAULT_FONT( "BoldCondensedItalic", 11) );
		Set( "Kismet.TypePicker.NormalFont", DEFAULT_FONT( "Regular", 11) );

		Set( "Kismet.GraphPicker.Title", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT("BoldCondensedItalic", 11) ) );

		Set( "Kismet.CreateBlueprint", new IMAGE_BRUSH( "/Icons/CreateBlueprint", Icon16x16) );
		Set( "Kismet.HarvestBlueprintFromActors", new IMAGE_BRUSH_SVG( "Starship/Blueprints/HarvestBlueprintFromActors", Icon16x16) );

		Set( "Kismet.Comment.Background", new IMAGE_BRUSH( "Old/Kismet2/Comment_Background", FVector2f(100.0f, 68.0f)) );

		Set( "Kismet.AllClasses.VariableIcon", new IMAGE_BRUSH_SVG("Starship/Blueprints/pill", Icon16x16));
		Set( "Kismet.AllClasses.ArrayVariableIcon", new IMAGE_BRUSH_SVG("Starship/Blueprints/pillarray", Icon16x16));
		Set( "Kismet.AllClasses.SetVariableIcon", new IMAGE_BRUSH_SVG("Starship/Blueprints/pillset", Icon16x16));
		Set( "Kismet.AllClasses.MapValueVariableIcon", new IMAGE_BRUSH_SVG("Starship/Blueprints/pillmapvalue", Icon16x16));
		Set( "Kismet.AllClasses.MapKeyVariableIcon", new IMAGE_BRUSH_SVG("Starship/Blueprints/pillmapkey", Icon16x16));

		Set( "Kismet.AllClasses.FunctionIcon", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_Blueprint_Function", Icon16x16));

		Set( "BlueprintEditor.ResetCamera", new IMAGE_BRUSH_SVG("Starship/Common/ResetCamera", Icon16x16));
		Set( "BlueprintEditor.ShowFloor", new IMAGE_BRUSH_SVG( "Starship/Common/Floor", Icon16x16));
		Set( "BlueprintEditor.ShowGrid", new IMAGE_BRUSH_SVG( "Starship/Common/Grid", Icon16x16 ) );
		Set( "BlueprintEditor.EnableSimulation", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20x20));
		Set( "BlueprintEditor.EnableProfiling", new IMAGE_BRUSH_SVG("Starship/Common/Statistics", Icon20x20) );
		Set( "SCS.NativeComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/SceneComponent_16", Icon16x16));
		Set( "SCS.Component", new IMAGE_BRUSH_SVG("Starship/AssetIcons/ActorComponent_16", Icon16x16));

		// curve viewer
		Set("AnimCurveViewer.MorphTargetOn", new IMAGE_BRUSH(TEXT("Persona/AnimCurveViewer/MorphTarget_On"), Icon16x16));
		Set("AnimCurveViewer.MaterialOn", new IMAGE_BRUSH(TEXT("Persona/AnimCurveViewer/Material_On"), Icon16x16));
		Set("AnimCurveViewer.MorphTargetOff", new IMAGE_BRUSH(TEXT("Persona/AnimCurveViewer/MorphTarget_Off"), Icon16x16));
		Set("AnimCurveViewer.MaterialOff", new IMAGE_BRUSH(TEXT("Persona/AnimCurveViewer/Material_Off"), Icon16x16));
		Set("AnimCurveViewer.MorphTargetHover", new IMAGE_BRUSH(TEXT("Persona/AnimCurveViewer/MorphTarget_On"), Icon16x16));
		Set("AnimCurveViewer.MaterialHover", new IMAGE_BRUSH(TEXT("Persona/AnimCurveViewer/Material_On"), Icon16x16));
		Set("AnimCurveViewer.ActiveCurveFont", DEFAULT_FONT("Bold", 8));

		// blend space
		Set("BlendSpaceEditor.ToggleTriangulation", new IMAGE_BRUSH(TEXT("Persona/BlendSpace/triangulation_16"), Icon16x16));
		Set("BlendSpaceEditor.ToggleLabels", new IMAGE_BRUSH(TEXT("Persona/BlendSpace/label_16"), Icon16x16));
		Set("BlendSpaceEditor.ArrowDown", new IMAGE_BRUSH(TEXT("Persona/BlendSpace/arrow_down_12x"), FVector2f(13.0f, 25.0f)));
		Set("BlendSpaceEditor.ArrowUp", new IMAGE_BRUSH(TEXT("Persona/BlendSpace/arrow_up_12x"), FVector2f(13.0f, 25.0f)));
		Set("BlendSpaceEditor.ArrowRight", new IMAGE_BRUSH(TEXT("Persona/BlendSpace/arrow_right_12x"), FVector2f(25.0f, 13.0f)));
		Set("BlendSpaceEditor.ArrowLeft", new IMAGE_BRUSH(TEXT("Persona/BlendSpace/arrow_left_12x"), FVector2f(25.0f, 13.0f)));
		Set("BlendSpaceEditor.PreviewIcon", new IMAGE_BRUSH(TEXT("Persona/BlendSpace/preview_21x"), FVector2f(21.0f, 21.0f)));
		Set("BlendSpaceEditor.LabelBackground", new FSlateRoundedBoxBrush(FStyleColors::Background, FStyleColors::Foreground, 1.0f));
		Set("BlendSpaceEditor.ZoomToFit", new IMAGE_BRUSH("GenericCurveEditor/Icons/FramingSelected_48x", Icon16x16));

		// Asset player slider
		FSliderStyle AssetPlayerSliderStyle = FSliderStyle()
			.SetNormalBarImage(   FSlateRoundedBoxBrush(FStyleColors::InputOutline, 2.0f, FStyleColors::InputOutline, 1.0f))
			.SetHoveredBarImage(  FSlateRoundedBoxBrush(FStyleColors::InputOutline, 2.0f, FStyleColors::InputOutline, 1.0f))
			.SetNormalThumbImage(  FSlateRoundedBoxBrush(FStyleColors::Foreground, Icon8x8) )
			.SetHoveredThumbImage( FSlateRoundedBoxBrush(FStyleColors::ForegroundHover, Icon8x8) )
			.SetBarThickness(4.0f);
		Set("AnimBlueprint.AssetPlayerSlider", AssetPlayerSliderStyle);
		
		const FButtonStyle BlueprintContextTargetsButtonStyle = FButtonStyle()
			.SetNormal(IMAGE_BRUSH("Common/TreeArrow_Collapsed_Hovered", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)))
			.SetHovered(IMAGE_BRUSH("Common/TreeArrow_Collapsed_Hovered", Icon10x10, FLinearColor(0.4f, 0.4f, 0.4f, 1.f)))
			.SetPressed(IMAGE_BRUSH("Common/TreeArrow_Collapsed_Hovered", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)));
// 		const FCheckBoxStyle BlueprintContextTargetsButtonStyle = FCheckBoxStyle()
// 			.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
// 			.SetUncheckedImage(IMAGE_BRUSH("Common/TreeArrow_Collapsed", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)))
// 			.SetUncheckedHoveredImage(IMAGE_BRUSH("Common/TreeArrow_Collapsed", Icon10x10, FLinearColor(0.4f, 0.4f, 0.4f, 1.f)))
// 			.SetUncheckedPressedImage(IMAGE_BRUSH("Common/TreeArrow_Collapsed", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)))
// 			.SetCheckedImage(IMAGE_BRUSH("Common/TreeArrow_Collapsed_Hovered", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)))
// 			.SetCheckedHoveredImage(IMAGE_BRUSH("Common/TreeArrow_Collapsed_Hovered", Icon10x10, FLinearColor(0.4f, 0.4f, 0.4f, 1.f)))
// 			.SetCheckedPressedImage(IMAGE_BRUSH("Common/TreeArrow_Collapsed_Hovered", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)));
		Set("BlueprintEditor.ContextMenu.TargetsButton", BlueprintContextTargetsButtonStyle);

		Set( "BlueprintEditor.CompactPinTypeSelector", FButtonStyle()
			.SetNormal        ( FSlateNoResource() )
			.SetPressed       ( BOX_BRUSH( "Common/Button_Pressed", 8.0f/32.0f, SelectionColor_Pressed ) )
			.SetHovered       ( BOX_BRUSH( "Common/Button_Hovered", 8.0f/32.0f, SelectionColor ) )
			.SetNormalPadding ( FMargin( 0,0,0,0 ) )
			.SetPressedPadding( FMargin( 1,1,2,2 ) )
			);

		const FComboButtonStyle& SimpleComboButton = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton");
		
		Set("BlueprintEditor.CompactVariableTypeSelector", 
			FComboButtonStyle(SimpleComboButton)
			.SetDownArrowPadding(0)
			.SetButtonStyle(
				FButtonStyle(SimpleComboButton.ButtonStyle)
				.SetNormalPadding(FMargin(0, 2, 2, 2))
				.SetPressedPadding(FMargin(0, 3, 2, 1)))
		);
	}

	// Access modifier display in MyBlueprint window for functions/variables
	{
		Set("BlueprintEditor.AccessModifier.Public", FTextBlockStyle().SetFont(DEFAULT_FONT("Bold", 9) ));
		Set("BlueprintEditor.AccessModifier.Default", FTextBlockStyle().SetFont(DEFAULT_FONT("Regular", 9)));
	}

	// Kismet linear expression display
	{
		Set( "KismetExpression.ReadVariable.Body", new BOX_BRUSH( "/Graph/Linear_VarNode_Background", FMargin(16.f/64.f, 12.f/28.f) ) );
		Set( "KismetExpression.ReadVariable", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 9 ) ) );
		Set( "KismetExpression.ReadVariable.Gloss", new BOX_BRUSH( "/Graph/Linear_VarNode_Gloss", FMargin(16.f/64.f, 12.f/28.f) ) );
		
		Set( "KismetExpression.ReadAutogeneratedVariable.Body", new BOX_BRUSH( "/Graph/Linear_VarNode_Background", FMargin(16.f/64.f, 12.f/28.f) ) );
		Set( "KismetExpression.ReadAutogeneratedVariable", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 9 ) ) );

		Set( "KismetExpression.OperatorNode", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "BoldCondensed", 20 ) ) );
		Set( "KismetExpression.FunctionNode", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Bold", 10 ) ) );
		Set( "KismetExpression.LiteralValue", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Bold", 10 ) ) );

	}

	//Find Results
	{
		Set("FindResults.FindInBlueprints", FTextBlockStyle(NormalText)
			.SetFont(REGULAR_ICON_FONT(10))
			.SetColorAndOpacity(FLinearColor::White)
		);
	}

	//Bookmarks
	{
		Set("Bookmarks.AddFolderButtonIcon", new IMAGE_BRUSH("Icons/icon_AddFolder_16x", Icon16x16));
		Set("Bookmarks.TreeViewItemFont", DEFAULT_FONT("Fonts/Roboto-Regular", 10));
		Set("Bookmarks.TreeViewRootItemFont", DEFAULT_FONT("Fonts/Roboto-Regular", 11));
	}

	//Blueprint Diff
	{
		Set("BlueprintDiff.ToolbarIcon", new IMAGE_BRUSH_SVG("Starship/Blueprints/BlueprintDiff", Icon16x16));
		Set("BlueprintDif.HasGraph", new IMAGE_BRUSH_SVG("Starship/Blueprints/blueprint_Dif_has_graph", Icon10x10));
		Set("BlueprintDif.MissingGraph", new IMAGE_BRUSH_SVG("Starship/Blueprints/blueprint_Dif_missing_graph", Icon8x8));
		Set("BlueprintDif.NextDiff", new IMAGE_BRUSH_SVG("/Starship/Blueprints/diff_next", Icon16x16));
		Set("BlueprintDif.PrevDiff", new IMAGE_BRUSH_SVG("/Starship/Blueprints/diff_prev", Icon16x16));
		Set("BlueprintDif.HorizontalDiff", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_horizontal_diff_view", Icon16x16));
		Set("BlueprintDif.VerticalDiff", new IMAGE_BRUSH_SVG("Starship/Blueprints/icon_vertical_diff_view", Icon16x16));
		Set("BlueprintDif.CopyPropertyLeft", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/arrow-left", Icon16x16));
		Set("BlueprintDif.CopyPropertyRight", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/arrow-right", Icon16x16));

		Set("BlueprintDif.ItalicText", 
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Italic", 10))
			.SetColorAndOpacity(FLinearColor(.7f, .7f, .7f))
		);
	}

	//Blueprint Merge
	{
		Set("BlueprintMerge.NextDiff", new IMAGE_BRUSH_SVG("Starship/Blueprints/diff_next", Icon16x16));
		Set("BlueprintMerge.PrevDiff", new IMAGE_BRUSH_SVG("Starship/Blueprints/diff_prev", Icon16x16));

		Set("BlueprintMerge.Finish", new IMAGE_BRUSH("/Icons/LV_Save", Icon16x16));

		Set("BlueprintMerge.Cancel", new IMAGE_BRUSH("/Icons/LV_Remove", Icon16x16));

		Set("BlueprintMerge.AcceptSource", new IMAGE_BRUSH("/Icons/AcceptMergeSource_40x", Icon16x16));

		Set("BlueprintMerge.AcceptTarget", new IMAGE_BRUSH("/Icons/AcceptMergeTarget_40x", Icon16x16));

		Set("BlueprintMerge.StartMerge", new IMAGE_BRUSH("/Icons/StartMerge_42x", Icon16x16));

	}

	// Play in editor / play in world
	{
		// Leftmost button for backplate style toolbar buttons
		FToolBarStyle MainToolbarLeftButton = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("AssetEditorToolbar");

		const FButtonStyle LeftToolbarButton = FButtonStyle(MainToolbarLeftButton.ButtonStyle)
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(4.0f, 0.0f, 0.0f, 4.0f)))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(4.0f, 0.0f, 0.0f, 4.0f)))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(4.0f, 0.0f, 0.0f, 4.0f)))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(4.0f, 0.0f, 0.0f, 4.0f)))
			.SetNormalPadding(FMargin(8.f, 2.f, 6.f, 2.f))
			.SetPressedPadding(FMargin(8.f, 2.f, 6.f, 2.f));


		MainToolbarLeftButton.SetButtonStyle(LeftToolbarButton);
		MainToolbarLeftButton.SetButtonPadding(FMargin(10.f, 0.0f, 0.0f, 0.0f));
		MainToolbarLeftButton.SetSeparatorPadding(FMargin(0.f, 0.f, 8.f, 0.f));

		Set("Toolbar.BackplateLeft", MainToolbarLeftButton);

		// Specialized Play Button (Left button with green color)
		FLinearColor GreenHSV = FStyleColors::AccentGreen.GetSpecifiedColor().LinearRGBToHSV();
		FLinearColor GreenHover = FLinearColor(GreenHSV.R, GreenHSV.G * .5f, GreenHSV.B, GreenHSV.A).HSVToLinearRGB();
		FLinearColor GreenPress = FLinearColor(GreenHSV.R, GreenHSV.G, GreenHSV.B*.5f, GreenHSV.A).HSVToLinearRGB(); 

		FToolBarStyle MainToolbarPlayButton = MainToolbarLeftButton;

		const FButtonStyle PlayToolbarButton = FButtonStyle(MainToolbarPlayButton.ButtonStyle)
			.SetNormalForeground(FStyleColors::AccentGreen)
			.SetPressedForeground(GreenPress)
			.SetHoveredForeground(GreenHover);

		MainToolbarPlayButton.SetButtonStyle(PlayToolbarButton);

		Set("Toolbar.BackplateLeftPlay", MainToolbarPlayButton);

		// Center Buttons for backplate style toolbar buttons
		FToolBarStyle MainToolbarCenterButton = MainToolbarLeftButton;

		const FButtonStyle CenterToolbarButton = FButtonStyle(MainToolbarCenterButton.ButtonStyle)
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 0.0f, 0.0f, 0.0f)))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 0.0f, 0.0f, 0.0f)))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 0.0f, 0.0f, 0.0f)))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 0.0f, 0.0f, 0.0f)))
			.SetNormalPadding(FMargin(2.f, 2.f, 6.f, 2.f))
			.SetPressedPadding(FMargin(2.f, 2.f, 6.f, 2.f));

		MainToolbarCenterButton.SetButtonPadding(0.0f);
		MainToolbarCenterButton.SetButtonStyle(CenterToolbarButton);

		Set("Toolbar.BackplateCenter", MainToolbarCenterButton);

		// Specialized Stop Button (Center button + Red color)

		FLinearColor RedHSV = FStyleColors::AccentRed.GetSpecifiedColor().LinearRGBToHSV();

		FLinearColor RedHover = FLinearColor(RedHSV.R, RedHSV.G * .5f, RedHSV.B, RedHSV.A).HSVToLinearRGB();
		FLinearColor RedPress = FLinearColor(RedHSV.R, RedHSV.G, RedHSV.B * .5f, RedHSV.A).HSVToLinearRGB();

		FToolBarStyle MainToolbarStopButton = MainToolbarCenterButton;

		const FButtonStyle StopToolbarButton = FButtonStyle(MainToolbarStopButton.ButtonStyle)
			.SetNormalForeground(FStyleColors::AccentRed)
			.SetPressedForeground(RedPress)
			.SetHoveredForeground(RedHover);

		MainToolbarStopButton.SetButtonStyle(StopToolbarButton);

		Set("Toolbar.BackplateCenterStop", MainToolbarStopButton);

		// Rightmost button for backplate style toolbar buttons
		FToolBarStyle MainToolbarRightButton = MainToolbarLeftButton;

		const FButtonStyle RightToolbarButton = FButtonStyle(MainToolbarRightButton.ButtonStyle)
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f)))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f)))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f)))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f)))
			.SetNormalPadding(FMargin(2.f, 2.f, 8.f, 2.f))
			.SetPressedPadding(FMargin(2.f, 2.f, 8.f, 2.f));

		MainToolbarRightButton.SetButtonStyle(RightToolbarButton);
		MainToolbarRightButton.SetButtonPadding(FMargin(0.0f, 0.0f, 4.0f, 0.0f));
		MainToolbarRightButton.SetSeparatorPadding(FMargin(4.f, -5.f, 8.f, -5.f));

		Set("Toolbar.BackplateRight", MainToolbarRightButton);

		// Rightmost button for backplate style toolbar buttons as a combo button
		FToolBarStyle MainToolbarRightComboButton = MainToolbarLeftButton;

		const FButtonStyle RightToolbarComboButton = FButtonStyle(MainToolbarRightComboButton.ButtonStyle)
			.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f)))
			.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f)))
			.SetPressed(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f)))
			.SetDisabled(FSlateRoundedBoxBrush(FStyleColors::Dropdown, FVector4(0.0f, 4.0f, 4.0f, 0.0f)))
			.SetNormalPadding(FMargin(7.f, 2.f, 6.f, 2.f))
			.SetPressedPadding(FMargin(7.f, 2.f, 6.f, 2.f));

		FComboButtonStyle PlayToolbarComboButton =
			FComboButtonStyle(FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FComboButtonStyle>("ComboButton"))
				.SetDownArrowPadding(FMargin(1.f, 0.f, 2.f, 0.f))
				.SetDownArrowImage(CORE_IMAGE_BRUSH_SVG("Starship/Common/ellipsis-vertical-narrow", FVector2f(6.f, 24.f)));
		PlayToolbarComboButton.ButtonStyle = RightToolbarComboButton;

		MainToolbarRightComboButton.SetButtonStyle(RightToolbarComboButton);
		MainToolbarRightComboButton.SetComboButtonStyle(PlayToolbarComboButton);
		MainToolbarRightComboButton.SetSeparatorPadding(FMargin(8.f, 0.f, 8.f, 0.f));
		MainToolbarRightComboButton.SetComboButtonPadding(FMargin(1.0f, 0.0f, 8.0f, 0.0f));

		Set("Toolbar.BackplateRightCombo", MainToolbarRightComboButton);

		Set("PlayWorld.Simulate", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20x20));
		Set( "PlayWorld.RepeatLastPlay", new IMAGE_BRUSH_SVG("Starship/Common/play", Icon20x20));
		Set( "PlayWorld.PlayInViewport", new IMAGE_BRUSH_SVG("Starship/Common/play", Icon20x20));
		Set("PlayWorld.PlaySimulate", new IMAGE_BRUSH_SVG("Starship/MainToolbar/PlaySimulate_20", Icon20x20));

		Set( "PlayWorld.PlayInEditorFloating", new IMAGE_BRUSH_SVG("Starship/MainToolbar/PlayNewEditorWindow", Icon20x20));
		Set( "PlayWorld.PlayInVR", new IMAGE_BRUSH_SVG("Starship/MainToolbar/PlayVRPreview", Icon20x20));
		Set( "PlayWorld.PlayInMobilePreview", new IMAGE_BRUSH_SVG("Starship/MainToolbar/PlayMobilePreview", Icon20x20));
		Set( "PlayWorld.PlayInNewProcess", new IMAGE_BRUSH_SVG("Starship/MainToolbar/PlayStandaloneGame", Icon20x20));

		Set( "PlayWorld.RepeatLastLaunch", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/launch", Icon20x20 ) );

		Set( "PlayWorld.PlayInCameraLocation", new IMAGE_BRUSH_SVG( "Starship/AssetIcons/CameraActor_16", Icon20x20 ) );
		Set( "PlayWorld.PlayInDefaultPlayerStart", new IMAGE_BRUSH_SVG( "Starship/AssetIcons/PlayerStart_16", Icon20x20 ) );

		Set("PlayWorld.ResumePlaySession", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon40x40));
		Set("PlayWorld.ResumePlaySession.Small", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20x20));
		Set( "PlayWorld.PausePlaySession", new IMAGE_BRUSH_SVG("Starship/MainToolbar/pause", Icon40x40));
		Set( "PlayWorld.PausePlaySession.Small", new IMAGE_BRUSH_SVG("Starship/MainToolbar/pause", Icon20x20));
		Set( "PlayWorld.SingleFrameAdvance", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/SingleFrameAdvance", Icon40x40 ) );
		Set( "PlayWorld.SingleFrameAdvance.Small", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/SingleFrameAdvance", Icon20x20 ) );

		Set( "PlayWorld.StopPlaySession", new CORE_IMAGE_BRUSH_SVG("Starship/Common/stop", Icon40x40));
		Set( "PlayWorld.StopPlaySession.Small", new CORE_IMAGE_BRUSH_SVG("Starship/Common/stop", Icon20x20));

		Set("PlayWorld.LateJoinSession", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon40x40));
		Set("PlayWorld.LateJoinSession.Small", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20x20));

		Set( "PlayWorld.PossessPlayer", new IMAGE_BRUSH_SVG("Starship/AssetIcons/PlayerController_16", Icon20x20));
		Set("PlayWorld.PlayPossess", new IMAGE_BRUSH_SVG("Starship/MainToolbar/PlayPossess_20", Icon20x20));
		Set( "PlayWorld.EjectFromPlayer", new IMAGE_BRUSH_SVG("Starship/MainToolbar/eject", Icon40x40));

		Set( "PlayWorld.ShowCurrentStatement", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/DebugFindNode", Icon40x40 ) );
		Set( "PlayWorld.ShowCurrentStatement.Small", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/DebugFindNode", Icon20x20 ) );
		Set( "PlayWorld.AbortExecution", new IMAGE_BRUSH_SVG("Starship/MainToolbar/DebugAbortExecution", Icon40x40));
		Set( "PlayWorld.AbortExecution.Small", new IMAGE_BRUSH_SVG("Starship/MainToolbar/DebugAbortExecution", Icon20x20));
		Set( "PlayWorld.ContinueExecution", new IMAGE_BRUSH_SVG("Starship/MainToolbar/DebugContinueExecution", Icon40x40));
		Set( "PlayWorld.ContinueExecution.Small", new IMAGE_BRUSH_SVG("Starship/MainToolbar/DebugContinueExecution", Icon20x20));
		Set( "PlayWorld.StepOut", new IMAGE_BRUSH_SVG("Starship/MainToolbar/DebugStepOut", Icon40x40));
		Set( "PlayWorld.StepOut.Small", new IMAGE_BRUSH_SVG("Starship/MainToolbar/DebugStepOut", Icon20x20));
		Set( "PlayWorld.StepInto", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/DebugStepInto", Icon40x40 ) );
		Set( "PlayWorld.StepInto.Small", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/DebugStepInto", Icon20x20 ) );
		Set( "PlayWorld.StepOver", new IMAGE_BRUSH_SVG( "Starship/MainToolbar/DebugStepOver", Icon40x40 ) );
		Set( "PlayWorld.StepOver.Small", new IMAGE_BRUSH_SVG("Starship/MainToolbar/DebugStepOver", Icon20x20));
	}


	// Kismet 2 debugger
	{
		Set( "Kismet.Breakpoint.Disabled", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Breakpoint_Disabled", Icon16x16, FStyleColors::AccentRed ) );
		Set( "Kismet.Breakpoint.EnabledAndInvalid", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Breakpoint_Invalid", Icon16x16, FStyleColors::AccentRed) );
		Set( "Kismet.Breakpoint.EnabledAndValid", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Breakpoint_Valid", Icon16x16, FStyleColors::AccentRed) );
		Set( "Kismet.Breakpoint.MixedStatus", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Breakpoint_Mixed", Icon16x16, FStyleColors::AccentRed) );
		
		Set( "Kismet.WatchIcon", new IMAGE_BRUSH_SVG( "Starship/GraphEditors/WatchVariable", Icon16x16 ) );
		Set( "Kismet.LatentActionIcon", new IMAGE_BRUSH_SVG( "Starship/Common/Timecode", Icon16x16 ) );

		Set( "Kismet.Trace.CurrentIndex", new IMAGE_BRUSH_SVG( "Starship/Common/NextArrow", Icon16x16 ) );
		Set( "Kismet.Trace.PreviousIndex", new IMAGE_BRUSH_SVG( "Starship/Common/PreviousArrow", Icon16x16 ) );

		Set( "Kismet.DebuggerOverlay.Breakpoint.Disabled", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Breakpoint_Disabled", Icon32x32, FStyleColors::AccentRed) );
		Set( "Kismet.DebuggerOverlay.Breakpoint.EnabledAndInvalid", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Breakpoint_Invalid", Icon32x32, FStyleColors::AccentRed) );
		Set( "Kismet.DebuggerOverlay.Breakpoint.EnabledAndValid", new IMAGE_BRUSH_SVG( "Starship/Blueprints/Breakpoint_Valid", Icon32x32, FStyleColors::AccentRed) );

		Set("Kismet.DebuggerOverlay.InstructionPointer", new IMAGE_BRUSH_SVG( "Starship/Blueprints/IP_Breakpoint", FVector2f(128.f,96.f)) );
		Set("Kismet.DebuggerOverlay.InstructionPointerBreakpoint", new IMAGE_BRUSH_SVG( "Starship/Blueprints/IP_Breakpoint", FVector2f(128.f,96.f), FStyleColors::AccentRed) );

		Set("Kismet.CallStackViewer.CurrentStackFrame", new IMAGE_BRUSH_SVG( "Starship/Blueprints/DebuggerArrow", Icon12x12 ));
		Set("Kismet.CallStackViewer.CurrentStackFrameColor", FLinearColor(0.728f, 0.364f, 0.003f) );
		Set("Kismet.CallStackViewer.LastStackFrameNavigatedToColor", FLinearColor( 0.4f, 0.5f, 0.7f, 1.0f ) );
	}

	// Asset context menu
	{
		Set("Persona.AssetActions.CreateAnimAsset", new IMAGE_BRUSH_SVG("Starship/Persona/PersonaCreateAsset", Icon20x20));
		Set("Persona.AssetActions.ReimportAnim", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));
		Set("Persona.AssetActions.Retarget", new IMAGE_BRUSH(TEXT("Icons/icon_Persona_Retarget_16x"), Icon16x16));
		Set("Persona.AssetActions.RetargetSkeleton", new IMAGE_BRUSH(TEXT("Icons/icon_Animation_Retarget_Skeleton_16x"), Icon16x16));
		Set("Persona.AssetActions.FindSkeleton", new IMAGE_BRUSH(TEXT("Icons/icon_Genericfinder_16x"), Icon16x16));
		Set("Persona.AssetActions.DuplicateAndRetargetSkeleton", new IMAGE_BRUSH(TEXT("Icons/icon_Animation_Duplicate_Retarget_Skeleton_16x"), Icon16x16));
		Set("Persona.AssetActions.AssignSkeleton", new IMAGE_BRUSH(TEXT("Icons/icon_Animation_Assign_Skeleton_16x"), Icon16x16));
	}

	// Blend space colors
	{
		Set("BlendSpaceKey.Regular", DefaultForeground);
		Set("BlendSpaceKey.Highlight", SelectionColor);
		Set("BlendSpaceKey.Pressed", SelectionColor_Pressed);
		Set("BlendSpaceKey.Drag", SelectionColor_Subdued);
		Set("BlendSpaceKey.Drop", SelectionColor_Inactive);
		Set("BlendSpaceKey.Invalid", FStyleColors::Warning);
		Set("BlendSpaceKey.Preview", FStyleColors::AccentGreen);
	}

	// Custom menu style for recent commands list
	{
		Set( "PinnedCommandList.Background", new BOX_BRUSH( "Common/RoundedSelection_16x", FMargin(4.0f/16.0f), FLinearColor( 0.2f, 0.2f, 0.2f, 0.2f ) ) );
		Set( "PinnedCommandList.Icon", new IMAGE_BRUSH( "Icons/icon_tab_toolbar_16px", Icon16x16 ) );
		Set( "PinnedCommandList.Expand", new IMAGE_BRUSH( "Icons/toolbar_expand_16x", Icon16x16) );
		Set( "PinnedCommandList.SubMenuIndicator", new IMAGE_BRUSH( "Common/SubmenuArrow", Icon8x8 ) );
		Set( "PinnedCommandList.SToolBarComboButtonBlock.Padding", FMargin(4.0f));
		Set( "PinnedCommandList.SToolBarButtonBlock.Padding", FMargin(4.0f));
		Set( "PinnedCommandList.SToolBarCheckComboButtonBlock.Padding", FMargin(4.0f));
		Set( "PinnedCommandList.SToolBarButtonBlock.CheckBox.Padding", FMargin(0.0f) );
		Set( "PinnedCommandList.SToolBarComboButtonBlock.ComboButton.Color", DefaultForeground );

		Set( "PinnedCommandList.Block.IndentedPadding", FMargin( 0.0f, 0.0f, 0.0f, 0.0f ) );
		Set( "PinnedCommandList.Block.Padding", FMargin( 0.0f, 0.0f, 0.0f, 0.0f ) );

		Set( "PinnedCommandList.Separator", new BOX_BRUSH( "Old/Button", 4.0f/32.0f ) );
		Set( "PinnedCommandList.Separator.Padding", FMargin( 0.5f ) );

		Set( "PinnedCommandList.Label", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 9 ) ) );
		Set( "PinnedCommandList.EditableText", FEditableTextBoxStyle(NormalEditableTextBoxStyle) .SetFont( DEFAULT_FONT( "Regular", 9 ) ) );
		Set( "PinnedCommandList.Keybinding", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 8 ) ) );

		Set( "PinnedCommandList.Heading", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT( "Regular", 8))
			.SetColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f)));

		/* Set images for various SCheckBox states associated with menu check box items... */
		const FCheckBoxStyle BasicMenuCheckBoxStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/SmallCheckBox", Icon14x14 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14 ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked", Icon14x14 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked_Hovered", Icon14x14 ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined", Icon14x14 ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon14x14 ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) );
 
		/* ...and add the new style */
		Set( "PinnedCommandList.CheckBox", BasicMenuCheckBoxStyle );
						
		/* Read-only checkbox that appears next to a menu item */
		/* Set images for various SCheckBox states associated with read-only menu check box items... */
		const FCheckBoxStyle BasicMenuCheckStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Icons/Empty_14x", Icon14x14 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Icons/Empty_14x", Icon14x14 ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14 ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/SmallCheck", Icon14x14 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheck", Icon14x14 ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheck", Icon14x14 ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Icons/Empty_14x", Icon14x14 ) )
			.SetUndeterminedHoveredImage( FSlateNoResource() )
			.SetUndeterminedPressedImage( FSlateNoResource() );

		/* ...and add the new style */
		Set( "PinnedCommandList.Check", BasicMenuCheckStyle );

		/* This radio button is actually just a check box with different images */
		/* Set images for various Menu radio button (SCheckBox) states... */
		const FCheckBoxStyle BasicMenuRadioButtonStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, SelectionColor ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor_Pressed ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor_Pressed ) );

		/* ...and set new style */
		Set( "PinnedCommandList.RadioButton", BasicMenuRadioButtonStyle );

		/* Create style for "Menu.ToggleButton" widget ... */
		const FCheckBoxStyle MenuToggleButtonCheckBoxStyle = FCheckBoxStyle()
			.SetCheckBoxType( ESlateCheckBoxType::ToggleButton )
			.SetUncheckedImage( FSlateNoResource() )
			.SetUncheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) )
			.SetUncheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ) )
			.SetCheckedImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) )
			.SetCheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) )
			.SetCheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor ) );
		/* ... and add new style */
		Set( "PinnedCommandList.ToggleButton", MenuToggleButtonCheckBoxStyle );

		Set( "PinnedCommandList.Button", FButtonStyle( NoBorder )
			.SetNormal ( FSlateNoResource() )
			.SetPressed( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) )
			.SetHovered( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ) )
			.SetNormalPadding( FMargin(0,1) )
			.SetPressedPadding( FMargin(0,2,0,0) )
		);

		Set( "PinnedCommandList.Button.Checked", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) );
		Set( "PinnedCommandList.Button.Checked_Hovered", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) );
		Set( "PinnedCommandList.Button.Checked_Pressed", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor ) );

		/* The style of a menu bar button when it has a sub menu open */
		Set( "PinnedCommandList.Button.SubMenuOpen", new BORDER_BRUSH( "Common/Selection", FMargin(4.f/16.f), FLinearColor(0.10f, 0.10f, 0.10f) ) );
	}

	{
		Set( "ViewportPinnedCommandList.Background", new FSlateNoResource() );
		Set( "ViewportPinnedCommandList.Icon", new IMAGE_BRUSH( "Icons/icon_tab_toolbar_16px", Icon16x16 ) );
		Set( "ViewportPinnedCommandList.Expand", new IMAGE_BRUSH( "Icons/toolbar_expand_16x", Icon16x16) );
		Set( "ViewportPinnedCommandList.SubMenuIndicator", new IMAGE_BRUSH( "Common/SubmenuArrow", Icon8x8 ) );
		Set( "ViewportPinnedCommandList.SToolBarComboButtonBlock.Padding", FMargin(4.0f));
		Set( "ViewportPinnedCommandList.SToolBarButtonBlock.Padding", FMargin(4.0f));
		Set( "ViewportPinnedCommandList.SToolBarCheckComboButtonBlock.Padding", FMargin(4.0f));
		Set( "ViewportPinnedCommandList.SToolBarButtonBlock.CheckBox.Padding", FMargin(0.0f) );
		Set( "ViewportPinnedCommandList.SToolBarComboButtonBlock.ComboButton.Color", DefaultForeground );

		Set( "ViewportPinnedCommandList.Block.IndentedPadding", FMargin( 0.0f, 0.0f, 0.0f, 0.0f ) );
		Set( "ViewportPinnedCommandList.Block.Padding", FMargin( 0.0f, 0.0f, 0.0f, 0.0f ) );

		Set( "ViewportPinnedCommandList.Separator", new BOX_BRUSH( "Old/Button", 4.0f/32.0f ) );
		Set( "ViewportPinnedCommandList.Separator.Padding", FMargin( 0.5f ) );

		Set( "ViewportPinnedCommandList.Label", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Bold", 9 ) ).SetColorAndOpacity(FLinearColor::Black) );
		Set( "ViewportPinnedCommandList.EditableText", FEditableTextBoxStyle(NormalEditableTextBoxStyle) .SetFont( DEFAULT_FONT( "Bold", 9 ) ).SetForegroundColor(FLinearColor::Black) );
		Set( "ViewportPinnedCommandList.Keybinding", FTextBlockStyle(NormalText) .SetFont( DEFAULT_FONT( "Regular", 8 ) ).SetColorAndOpacity(FLinearColor::Gray) );

		Set( "ViewportPinnedCommandList.Heading", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT( "Regular", 8))
			.SetColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f, 1.0f)));

		/* Set images for various SCheckBox states associated with menu check box items... */
		const FCheckBoxStyle BasicMenuCheckBoxStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/SmallCheckBox", Icon14x14, FLinearColor::Black ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14, FLinearColor::Black ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked", Icon14x14, FLinearColor::Black ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked_Hovered", Icon14x14, FLinearColor::Black ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Checked_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined", Icon14x14, FLinearColor::Black ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon14x14, FLinearColor::Black ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/CheckBox_Undetermined_Hovered", Icon14x14, FLinearColor( 0.5f, 0.5f, 0.5f ) ) )
			.SetPadding(FMargin(2.0f))
			.SetForegroundColor(FLinearColor::Black);
 
		/* ...and add the new style */
		Set( "ViewportPinnedCommandList.CheckBox", BasicMenuCheckBoxStyle );
						
		/* Read-only checkbox that appears next to a menu item */
		/* Set images for various SCheckBox states associated with read-only menu check box items... */
		const FCheckBoxStyle BasicMenuCheckStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Icons/Empty_14x", Icon14x14, FLinearColor::Black ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Icons/Empty_14x", Icon14x14, FLinearColor::Black ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheckBox_Hovered", Icon14x14, FLinearColor::Black ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/SmallCheck", Icon14x14, FLinearColor::Black ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/SmallCheck", Icon14x14, FLinearColor::Black ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/SmallCheck", Icon14x14, FLinearColor::Black ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Icons/Empty_14x", Icon14x14, FLinearColor::Black ) )
			.SetUndeterminedHoveredImage( FSlateNoResource() )
			.SetUndeterminedPressedImage( FSlateNoResource() )
			.SetPadding(FMargin(2.0f))
			.SetForegroundColor(FLinearColor::Black);

		/* ...and add the new style */
		Set( "ViewportPinnedCommandList.Check", BasicMenuCheckStyle );

		/* This radio button is actually just a check box with different images */
		/* Set images for various Menu radio button (SCheckBox) states... */
		const FCheckBoxStyle BasicMenuRadioButtonStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16 ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, SelectionColor ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor_Pressed ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16 ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, SelectionColor_Pressed ) )
			.SetPadding(FMargin(2.0f))
			.SetForegroundColor(FLinearColor::Black);

		/* ...and set new style */
		Set( "ViewportPinnedCommandList.RadioButton", BasicMenuRadioButtonStyle );

		/* Create style for "Menu.ToggleButton" widget ... */
		const FCheckBoxStyle MenuToggleButtonCheckBoxStyle = FCheckBoxStyle()
			.SetCheckBoxType( ESlateCheckBoxType::ToggleButton )
			.SetUncheckedImage( FSlateNoResource() )
			.SetUncheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor_Pressed ) )
			.SetUncheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x", 4.0f/16.0f, SelectionColor ) )
			.SetCheckedImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) )
			.SetCheckedHoveredImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) )
			.SetCheckedPressedImage( BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor ) )
			.SetPadding(FMargin(2.0f))
			.SetForegroundColor(FLinearColor::Black);
		/* ... and add new style */
		Set( "ViewportPinnedCommandList.ToggleButton", MenuToggleButtonCheckBoxStyle );

		const FButtonStyle ViewportBaseButtonStyle = FButtonStyle()
			.SetNormal(FSlateNoResource())
			.SetHovered(FSlateNoResource())
			.SetPressed(FSlateNoResource())
			.SetNormalPadding(FMargin(2, 2, 2, 3))
			.SetPressedPadding(FMargin(2, 3, 2, 2));

		Set( "ViewportPinnedCommandList.Button", FButtonStyle(ViewportBaseButtonStyle)
			.SetNormal(BOX_BRUSH( "Common/SmallRoundedButton", FMargin(7.f/16.f), FLinearColor(1,1,1,0.75f)))
			.SetHovered(BOX_BRUSH( "Common/SmallRoundedButton", FMargin(7.f/16.f), FLinearColor(1,1,1, 1.0f)))
			.SetPressed(BOX_BRUSH( "Common/SmallRoundedButton", FMargin(7.f/16.f)))
			.SetNormalPadding( FMargin(2,3) )
			.SetPressedPadding( FMargin(2,4,2,2) )
		);

		Set( "ViewportPinnedCommandList.Button.Checked", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) );
		Set( "ViewportPinnedCommandList.Button.Checked_Hovered", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor_Pressed ) );
		Set( "ViewportPinnedCommandList.Button.Checked_Pressed", new BOX_BRUSH( "Common/RoundedSelection_16x",  4.0f/16.0f, SelectionColor ) );

		/* The style of a menu bar button when it has a sub menu open */
		Set( "ViewportPinnedCommandList.Button.SubMenuOpen", new BORDER_BRUSH( "Common/Selection", FMargin(4.f/16.f), FLinearColor(0.10f, 0.10f, 0.10f) ) );

		Set( "ViewportPinnedCommandList.ComboButton", FComboButtonStyle()
			.SetButtonStyle(ViewportBaseButtonStyle)
			.SetDownArrowImage(IMAGE_BRUSH("Common/ComboArrow", Icon8x8))
			.SetMenuBorderBrush(BOX_BRUSH("Old/Menu_Background", FMargin(8.0f/64.0f)))
			.SetMenuBorderPadding(FMargin(0.0f))
		);
	}

	// Animation blueprint
	{
		Set("AnimGraph.Attribute.Border.Solid", new FSlateRoundedBoxBrush(FStyleColors::White));
		Set("AnimGraph.Attribute.Border.Outline", new FSlateRoundedBoxBrush(FStyleColors::Transparent, FStyleColors::White, 1.0f));
		Set("AnimGraph.Attribute.DefaultColor", FStyleColors::AccentGray);

		FSlateColor AttributeTextColor = FStyleColors::AccentBlack;
		Set("AnimGraph.Attribute.TextColor", AttributeTextColor);
		Set("AnimGraph.Attribute.Text", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", FStarshipCoreStyle::SmallTextSize))
			.SetColorAndOpacity(AttributeTextColor));

		Set("AnimGraph.Attribute.Pose.Color", FStyleColors::White);
		Set("AnimGraph.Attribute.Curves.Icon", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphCurves", Icon16x16));
		Set("AnimGraph.Attribute.Curves.Color", FStyleColors::AccentPurple);
		Set("AnimGraph.Attribute.Attributes.Icon", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphAttributes", Icon16x16));
		Set("AnimGraph.Attribute.Attributes.Color", FStyleColors::AccentYellow);
		Set("AnimGraph.Attribute.Sync.Icon", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphSync", Icon16x16));
		Set("AnimGraph.Attribute.Sync.Color", FStyleColors::AccentBlue);
		Set("AnimGraph.Attribute.RootMotionDelta.Icon", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphRootMotionDelta", Icon16x16));
		Set("AnimGraph.Attribute.RootMotionDelta.Color", FStyleColors::AccentGreen);
		Set("AnimGraph.Attribute.InertialBlending.Icon", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphInertialBlending", Icon16x16));
		Set("AnimGraph.Attribute.InertialBlending.Color", FStyleColors::AccentOrange);

		Set("AnimGraph.PoseWatch.Icon", new IMAGE_BRUSH_SVG("Starship/AnimationBlueprintEditor/AnimationGraphPoseWatch", Icon16x16));

		Set("AnimGraph.AnimNodeReference.Subtitle", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Italic", 8))
			.SetColorAndOpacity(FLinearColor(218.0f/255.0f,218.0f/255.0f,96.0f/255.0f, 0.5f))
		);

		FTextBlockStyle TagTextStyle = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Italic", 9))
			.SetColorAndOpacity(FLinearColor(218.0f/255.0f,218.0f/255.0f,96.0f/255.0f, 0.5f));
		
		Set("AnimGraph.Node.Tag", FInlineEditableTextBlockStyle(FCoreStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("InlineEditableTextBlockStyle"))
			.SetTextStyle(TagTextStyle));
	}
	
	// Property Access 
	{
		Set("PropertyAccess.CompiledContext.Text", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Italic", 8))
			.SetColorAndOpacity(FLinearColor(218.0f/255.0f,218.0f/255.0f,96.0f/255.0f, 0.5f))
		);

		Set("PropertyAccess.CompiledContext.Border", new FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, 2.0f));
	}
#endif // WITH_EDITOR
}

void FStarshipEditorStyle::FStyle::SetupClassThumbnailOverlays()
{
	Set("ClassThumbnailOverlays.SkeletalMesh_NeedSkinning", new IMAGE_BRUSH("Icons/AssetIcons/SkeletalMeshNeedSkinning_16x", Icon16x16));
}

void FStarshipEditorStyle::FStyle::SetupClassIconsAndThumbnails()
{
#if WITH_EDITOR
	// Actor Classes Outliner
	{
		struct FClassIconInfo
		{
			FClassIconInfo(const TCHAR* InType, bool bInHas64Size = true)
				: Type(InType)
				, bHas64Size(bInHas64Size)
			{}

			const TCHAR* Type;
			bool bHas64Size;
		};

		Set("ClassIcon.Light", new IMAGE_BRUSH("Icons/ActorIcons/LightActor_16x", Icon16x16));
		Set("ClassIcon.BrushAdditive", new IMAGE_BRUSH("Icons/ActorIcons/Brush_Add_16x", Icon16x16));
		Set("ClassIcon.BrushSubtractive", new IMAGE_BRUSH("Icons/ActorIcons/Brush_Subtract_16x", Icon16x16));
		Set("ClassIcon.Deleted", new IMAGE_BRUSH("Icons/ActorIcons/DeletedActor_16px", Icon16x16));

		// Component classes
	
		Set("ClassIcon.BlueprintCore", new IMAGE_BRUSH("Icons/AssetIcons/Blueprint_16x", Icon16x16));
		Set("ClassIcon.LightComponent", new IMAGE_BRUSH("Icons/ActorIcons/LightActor_16x", Icon16x16));
		Set("ClassIcon.ArrowComponent", new IMAGE_BRUSH("Icons/ActorIcons/Arrow_16px", Icon16x16));
		Set("ClassIcon.MaterialBillboardComponent", new IMAGE_BRUSH("Icons/ActorIcons/MaterialSprite_16px", Icon16x16));
		Set("ClassIcon.BillboardComponent", new IMAGE_BRUSH("Icons/ActorIcons/SpriteComponent_16px", Icon16x16));
		Set("ClassIcon.TimelineComponent", new IMAGE_BRUSH("Icons/ActorIcons/TimelineComponent_16px", Icon16x16));
		Set("ClassIcon.ChildActorComponent", new IMAGE_BRUSH("Icons/ActorIcons/ChildActorComponent_16px", Icon16x16));

		Set("ClassIcon.AudioComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Audio_16", Icon16x16));
		Set("ClassIcon.BoxComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/BoxCollision_16", Icon16x16));
		Set("ClassIcon.CapsuleComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/CapsuleCollision_16", Icon16x16));
		Set("ClassIcon.SphereComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/SphereCollision_16", Icon16x16));
		Set("ClassIcon.SplineComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Spline_16", Icon16x16));

		Set("ClassIcon.AtmosphericFogComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/AtmosphericFog_16", Icon16x16));
		Set("ClassIcon.BrushComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Brush_16", Icon16x16));
		Set("ClassIcon.CableComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/CableActor_16", Icon16x16));
		Set("ClassIcon.CameraComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/CameraActor_16", Icon16x16));
		Set("ClassIcon.DecalComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/DecalActor_16", Icon16x16));
		Set("ClassIcon.DirectionalLightComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/DirectionalLight_16", Icon16x16));
		Set("ClassIcon.ExponentialHeightFogComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/ExponentialHeightFog_16", Icon16x16));
		Set("ClassIcon.ForceFeedbackComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/ForceFeedbackEffect_16", Icon16x16));
		Set("ClassIcon.LandscapeComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Landscape_16", Icon16x16));
		Set("ClassIcon.ParticleSystemComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/Emitter_16", Icon16x16));
		Set("ClassIcon.PlanarReflectionComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/PlanarReflectionCapture_16", Icon16x16));
		Set("ClassIcon.PointLightComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/PointLight_16", Icon16x16));
		Set("ClassIcon.RectLightComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/RectLight_16", Icon16x16));
		Set("ClassIcon.RadialForceComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/RadialForceActor_16", Icon16x16));
		Set("ClassIcon.SceneCaptureComponent2D", new IMAGE_BRUSH_SVG("Starship/AssetIcons/SceneCapture2D_16", Icon16x16));
		Set("ClassIcon.SceneCaptureComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/SphereReflectionCapture_16", Icon16x16));
		Set("ClassIcon.SingleAnimSkeletalComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/SkeletalMesh_16", Icon16x16));
		Set("ClassIcon.SkyAtmosphereComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/SkyAtmosphere_16", Icon16x16));
		Set("ClassIcon.SkeletalMeshComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/SkeletalMesh_16", Icon16x16));
		Set("ClassIcon.SpotLightComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/SpotLight_16", Icon16x16));
		Set("ClassIcon.StaticMeshComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/StaticMesh_16", Icon16x16));
		Set("ClassIcon.TextRenderComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/TextRenderActor_16", Icon16x16));
		Set("ClassIcon.VectorFieldComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/VectorFieldVolume_16", Icon16x16));
		Set("ClassIcon.VolumetricCloudComponent", new IMAGE_BRUSH_SVG("Starship/AssetIcons/VolumetricCloud_16", Icon16x16));

		Set("ClassIcon.MovableMobilityIcon", new IMAGE_BRUSH("Icons/ActorIcons/Light_Movable_16x", Icon16x16));
		Set("ClassIcon.StationaryMobilityIcon", new IMAGE_BRUSH("Icons/ActorIcons/Light_Adjustable_16x", Icon16x16));
		Set("ClassIcon.ComponentMobilityHeaderIcon", new IMAGE_BRUSH("Icons/ActorIcons/ComponentMobilityHeader_7x16", Icon7x16));

		// Curve Editor icons
		Set("ClassIcon.CurveEditorEulerFilter", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/CurveEditorOpEuler", Icon16x16));

		// Asset Type Classes
		const TCHAR* AssetTypes[] = {

			TEXT("AbilitySystemComponent"),
			TEXT("AIPerceptionComponent"),
			TEXT("CameraAnim"),
			TEXT("Default"),
			TEXT("DirectionalLightMovable"),
			TEXT("DirectionalLightStatic"),
			TEXT("DirectionalLightStationary"),
			TEXT("FontFace"),
			TEXT("ForceFeedbackEffect"),
			TEXT("InterpData"),
			TEXT("LevelSequence"),
			TEXT("LightmassCharacterIndirectDetailVolume"),
			TEXT("MassiveLODOverrideVolume"),
			TEXT("MaterialParameterCollection"),
			TEXT("MultiFont"),
			TEXT("ParticleSystem"),
			TEXT("PhysicsConstraintComponent"),
			TEXT("PhysicsThrusterComponent"),
			TEXT("SkyLightComponent"),
			TEXT("SlateWidgetStyleAsset"),
			TEXT("StringTable"),
			TEXT("SpotLightMovable"),
			TEXT("SpotLightStatic"),
			TEXT("SpotLightStationary"),
			TEXT("Cube"),
			TEXT("Sphere"),
			TEXT("Cylinder"),
			TEXT("Cone"),
			TEXT("Plane"),
		};

		for (int32 TypeIndex = 0; TypeIndex < UE_ARRAY_COUNT(AssetTypes); ++TypeIndex)
		{
			const TCHAR* Type = AssetTypes[TypeIndex];
			Set( *FString::Printf(TEXT("ClassIcon.%s"), Type),		new IMAGE_BRUSH(FString::Printf(TEXT("Icons/AssetIcons/%s_%dx"), Type, 16), Icon16x16 ) );
			Set( *FString::Printf(TEXT("ClassThumbnail.%s"), Type),	new IMAGE_BRUSH(FString::Printf(TEXT("Icons/AssetIcons/%s_%dx"), Type, 64), Icon64x64 ) );
		}

		const FClassIconInfo AssetTypesSVG[] = {
			{TEXT("Actor")},
			{TEXT("ActorComponent")},
			{TEXT("AIController")},
			{TEXT("AimOffsetBlendSpace")},
			{TEXT("AimOffsetBlendSpace1D")},
			{TEXT("AmbientSound")},
			{TEXT("AnimationModifier")},
			{TEXT("AnimationSharingSetup")},
			{TEXT("AnimBlueprint")},
			{TEXT("AnimComposite")},
			{TEXT("AnimInstance")},
			{TEXT("AnimLayerInterface")},
			{TEXT("AnimMontage")},
			{TEXT("AnimSequence")},
			{TEXT("ApplicationLifecycleComponent")},
			{TEXT("AtmosphericFog")},
			{TEXT("AudioVolume")},
			{TEXT("BehaviorTree")},
			{TEXT("BlackboardData")},
			{TEXT("BlendSpace")},
			{TEXT("BlendSpace1D")},
			{TEXT("BlockingVolume")},
			{TEXT("Blueprint")},
			{TEXT("BlueprintFunctionLibrary")},
			{TEXT("BlueprintGeneratedClass")},
			{TEXT("BlueprintInterface")},
			{TEXT("BlueprintMacroLibrary")},
			{TEXT("BoxReflectionCapture")},
			{TEXT("Brush")},
			{TEXT("ButtonStyleAsset")},
			{TEXT("CableActor")},
			{TEXT("CameraActor")},
			{TEXT("CameraBlockingVolume")},
			{TEXT("CameraRig_Crane")},
			{TEXT("CameraRig_Rail")},
			{TEXT("Character")},
			{TEXT("CharacterMovementComponent")},
			{TEXT("CineCameraActor")},
			{TEXT("Class")},
			{TEXT("CompositingElement")},
			{TEXT("CullDistanceVolume")},
			{TEXT("CurveBase")},
			{TEXT("DataAsset")},
			{TEXT("DataTable")},
			{TEXT("DataLayerAsset")},
			{TEXT("DecalActor")},
			{TEXT("DefaultPawn")},
			{TEXT("DialogueVoice")},
			{TEXT("DialogueWave")},
			{TEXT("DirectionalLight")},
			{TEXT("DocumentationActor")},
			{TEXT("EditorUtilityBlueprint")},	
			{TEXT("EditorUtilityWidgetBlueprint")},
			{TEXT("EnvQuery")},
			{TEXT("Emitter")},
			{TEXT("EmptyActor")},
			{TEXT("ExponentialHeightFog")},
			{TEXT("ExternalDataLayerAsset")},
			{TEXT("FileMediaOutput")},
			{TEXT("FileMediaSource")},
			{TEXT("FoliageType_Actor")},
			{TEXT("Font")},
			{TEXT("ForceFeedback")},
			{TEXT("FusionPatch")},
			{TEXT("GameModeBase")},
			{TEXT("GameStateBase")},
			{TEXT("GeometryCollection")},
			{TEXT("GroupActor")},
			{TEXT("HierarchicalInstancedStaticMeshComponent")},
			{TEXT("HLODLayer")},
			{TEXT("HUD")},
			{TEXT("ImagePlate")},
			{TEXT("ImpulseResponse")},
			{TEXT("InstancedStaticMeshComponent")},
			{TEXT("Interface")},
			{TEXT("KillZVolume")},
			{TEXT("Landscape")},
			{TEXT("LandscapeEditLayer")},
			{TEXT("LandscapeEditLayerBase")},
			{TEXT("LandscapeEditLayerSplines")},
			{TEXT("LevelBounds")},
			{TEXT("LevelInstance")},
			{TEXT("LevelInstancePivot")},
			{TEXT("PackedLevelActor")},
			{TEXT("LevelScriptActor")},
			{TEXT("LevelSequenceActor")},
			{TEXT("LevelStreamingVolume")},
			{TEXT("LightmassCharacterDetailIndirectVolume")},
			{TEXT("LightmassImportanceVolume")},
			{TEXT("LightmassVolume")},
			{TEXT("LiveLinkPreset")},
			{TEXT("Material")},
			{TEXT("MaterialCacheVirtualTextureTag")},
			{TEXT("MaterialFunction")},
			{TEXT("MaterialInstanceActor")},
			{TEXT("MaterialInstanceConstant")},
			{TEXT("MediaPlayer")},
			{TEXT("MediaTexture")},
			{TEXT("MirrorDataTable")},
			{TEXT("ModularSynthPresetBank")},
			{TEXT("NavLink")},
			{TEXT("NavLinkProxy")},
			{TEXT("NavMeshBoundsVolume")},
			{TEXT("NavModifierComponent")},
			{TEXT("NavModifierVolume")},
			{TEXT("Note")},
			{TEXT("Object")},
			{TEXT("ObjectLibrary")},
			{TEXT("PainCausingVolume")},
			{TEXT("Pawn")},
			{TEXT("PawnNoiseEmitterComponent")},
			{TEXT("PawnSensingComponent")},
			{TEXT("PhysicalMaterial")},
			{TEXT("PhysicsAsset")},
			{TEXT("PhysicsConstraintActor")},
			{TEXT("PhysicsHandleComponent")},
			{TEXT("PhysicsThruster")},
			{TEXT("PhysicsVolume")},
			{TEXT("PlanarReflectionCapture")},
			{TEXT("PlatformMediaSource")},
			{TEXT("PlayerController")},
			{TEXT("PlayerStart")},
			{TEXT("PointLight")},
			{TEXT("PoseAsset")},
			{TEXT("PostProcessVolume")},
			{TEXT("PrecomputedVisibilityOverrideVolume")},
			{TEXT("PrecomputedVisibilityVolume")},
			{TEXT("ProceduralFoliageBlockingVolume")},
			{TEXT("ProceduralFoliageVolume")},
			{TEXT("ProjectileMovementComponent")},
			{TEXT("RadialForceActor")},
			{TEXT("RectLight")},
			{TEXT("ReflectionCapture")},
			{TEXT("ReverbEffect")},
			{TEXT("RotatingMovementComponent")},
			{TEXT("RuntimeVirtualTexture")},
			{TEXT("SceneCapture2D")},
			{TEXT("SceneCaptureCube")},
			{TEXT("SceneComponent")},
			{TEXT("SkeletalMesh")},
			{TEXT("SkeletalMeshActor")},
			{TEXT("Skeleton")},
			{TEXT("SkyAtmosphere")},
			{TEXT("SkyLight")},
			{TEXT("SlateBrushAsset")},
			{TEXT("SoundAttenuation")},
			{TEXT("SoundClass")},
			{TEXT("SoundConcurrency")},
			{TEXT("SoundCue")},
			{TEXT("SoundEffectSourcePreset")},
			{TEXT("SoundMix")},
			{TEXT("SoundSubmix") },
			{TEXT("SphereReflectionCapture")},
			{TEXT("SpecularProfile")},
			{TEXT("SpotLight")},
			{TEXT("SpringArmComponent")},
			{TEXT("StaticMesh")},
			{TEXT("StaticMeshActor")},
			{TEXT("StreamMediaSource")},
			{TEXT("SubsurfaceProfile")},
			{TEXT("TargetPoint")},
			{TEXT("TemplateSequence")},
			{TEXT("TextRenderActor")},
			{TEXT("Texture2D")},
			{TEXT("Texture2DArray")},
			{TEXT("TextureCube")},
			{TEXT("TextureCubeArray")},
			{TEXT("TextureRenderTarget2D")},
			{TEXT("TextureRenderTarget2DArray")},
			{TEXT("TextureRenderTargetCube")},
			{TEXT("TextureRenderTargetVolume")},
			{TEXT("TimeCodeSynchronizer")},
			{TEXT("TouchInterface")},
			{TEXT("ToonProfile")},
			{TEXT("TriggerBase")},
			{TEXT("TriggerBox")},
			{TEXT("TriggerCapsule")},
			{TEXT("TriggerSphere")},
			{TEXT("TriggerVolume")},
			{TEXT("UserDefinedCaptureProtocol")},
			{TEXT("UserDefinedEnum")},
			{TEXT("UserDefinedStruct") },
			{TEXT("UserWidget")},
			{TEXT("VectorField")},
			{TEXT("VectorFieldVolume")},
			{TEXT("VirtualTextureBuilder")},
			{TEXT("Volume")},
			{TEXT("VolumeTexture")},
			{TEXT("VolumetricCloud"), false},
			{TEXT("VolumetricLightmapDensityVolume")},
			{TEXT("WidgetBlueprint")},
			{TEXT("WidgetBlueprintGeneratedClass")},
			{TEXT("WindDirectionalSource")},
			{TEXT("World")},
			{TEXT("WorldDataLayers")}
		};
	
		// SVG Asset icons
		{
			for (int32 TypeIndex = 0; TypeIndex < UE_ARRAY_COUNT(AssetTypesSVG); ++TypeIndex)
			{
				const FClassIconInfo& Info = AssetTypesSVG[TypeIndex];

				// Look up if the brush already exists to audit old vs new icons during starship development.
				FString ClassIconName = FString::Printf(TEXT("ClassIcon.%s"), Info.Type);
				if (GetOptionalBrush(*ClassIconName, nullptr, nullptr))
				{
					UE_LOGF(LogSlate, Log, "%ls already found", *ClassIconName);
				}

				Set(*FString::Printf(TEXT("ClassIcon.%s"), Info.Type), new IMAGE_BRUSH_SVG(FString::Printf(TEXT("Starship/AssetIcons/%s_%d"), Info.Type, 16), Icon16x16));
				if (Info.bHas64Size)
				{
					Set(*FString::Printf(TEXT("ClassThumbnail.%s"), Info.Type), new IMAGE_BRUSH_SVG(FString::Printf(TEXT("Starship/AssetIcons/%s_%d"), Info.Type, 64), Icon64x64));
				}
				else
				{
					// Temp to avoid missing icons while in progress. use the 64 variant for 16 for now.  
					Set(*FString::Printf(TEXT("ClassThumbnail.%s"), Info.Type), new IMAGE_BRUSH_SVG(FString::Printf(TEXT("Starship/AssetIcons/%s_%d"), Info.Type, 16), Icon64x64));
				}
			}
		}
	}
#endif
}

void FStarshipEditorStyle::FStyle::SetupContentBrowserStyle()
{
#if WITH_EDITOR
	// Content Browser
	{
		// Tab and menu icon
		Set("ContentBrowser.TabIcon", new IMAGE_BRUSH_SVG("Starship/Common/ContentBrowser", Icon16x16));
		Set("ContentBrowser.PrivateContentEdit", new IMAGE_BRUSH("Icons/hiererchy_16x", Icon16x16));

		// Sources View
		Set("ContentBrowser.SourceTitleFont", DEFAULT_FONT( "Regular", 12 ) );

		Set("ContentBrowser.SourceTreeItemFont", FStarshipCoreStyle::GetCoreStyle().GetFontStyle("NormalFont"));
		Set("ContentBrowser.SourceTreeRootItemFont", FStarshipCoreStyle::GetCoreStyle().GetFontStyle("NormalFont"));

		Set("ContentBrowser.BreadcrumbPathPickerFolder", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", Icon16x16));

		Set("ContentBrowser.AssetTreeFolderClosed", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-closed", Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpen", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-open", Icon16x16));

		Set("ContentBrowser.AssetTreeFolderClosedVirtual", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-virtual-closed", Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpenVirtual", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-virtual-open", Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpenDeveloper", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-developer-open", Icon16x16));
		Set("ContentBrowser.AssetTreeFolderClosedDeveloper", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-developer", Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpenCode", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-code-open", Icon16x16));
		Set("ContentBrowser.AssetTreeFolderClosedCode", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-code", Icon16x16));
		Set("ContentBrowser.AssetTreeFolderOpenPluginRoot", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-pluginroot-open", Icon16x16));
		Set("ContentBrowser.AssetTreeFolderClosedPluginRoot", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder-pluginroot", Icon16x16));

		Set("ContentBrowser.DefaultFolderColor", FStyleColors::AccentFolder);

		// Asset list view
		Set( "ContentBrowser.AssetListViewNameFont", DEFAULT_FONT( "Regular", 12 ) );
		Set( "ContentBrowser.AssetListViewNameFontDirty", DEFAULT_FONT( "Bold", 12 ) );
		Set( "ContentBrowser.AssetListViewClassFont", DEFAULT_FONT( "Light", 10 ) );

		// Asset picker
		Set("ContentBrowser.NoneButton", FButtonStyle(Button)
			.SetNormal(FSlateNoResource())
			.SetHovered(BOX_BRUSH( "Common/Selection", 8.0f/32.0f, SelectionColor ))
			.SetPressed(BOX_BRUSH( "Common/Selection", 8.0f/32.0f, SelectionColor_Pressed ))
		);
		Set( "ContentBrowser.NoneButtonText", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "Regular", 12 ) )
			.SetColorAndOpacity( FLinearColor::White )
		);

		// Tile view
		Set( "ContentBrowser.AssetTileViewNameFont", DEFAULT_FONT("Regular", 9));
		Set( "ContentBrowser.AssetTileViewClassNameFont", DEFAULT_FONT("Regular", 7));
		Set( "ContentBrowser.AssetTileViewNameFontSmall", DEFAULT_FONT( "VeryLight", 8 ) );
		Set( "ContentBrowser.AssetTileViewNameFontVerySmall", DEFAULT_FONT( "VeryLight", 7 ) );
		Set( "ContentBrowser.AssetTileViewNameFontDirty", FStyleFonts::Get().SmallBold);

		Set("ContentBrowser.AssetListView.ColumnListTableRow", FTableRowStyle()
			.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Recessed))
			.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))
			.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Background))
			.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::SelectHover))
			.SetSelectorFocusedBrush( BORDER_BRUSH( "Common/Selector", FMargin(4.f/16.f), FStyleColors::Select ) )
			.SetActiveBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, FStyleColors::Select ) )
			.SetActiveHoveredBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, FStyleColors::Select ) )
			.SetInactiveBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, FStyleColors::SelectInactive ) )
			.SetInactiveHoveredBrush( IMAGE_BRUSH( "Common/Selection", Icon8x8, FStyleColors::SelectInactive ) )
			.SetTextColor( DefaultForeground )
			.SetSelectedTextColor( InvertedForeground )
			);

		Set("ContentBrowser.AssetListView.TileTableRow", FTableRowStyle()
			.SetEvenRowBackgroundBrush(FSlateNoResource() )
			.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
			.SetOddRowBackgroundBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(FSlateNoResource())
			.SetSelectorFocusedBrush(FSlateNoResource())
			.SetActiveBrush(FSlateNoResource())
			.SetActiveHoveredBrush(FSlateNoResource())
			.SetInactiveBrush(FSlateNoResource())
			.SetInactiveHoveredBrush(FSlateNoResource())
			.SetTextColor(DefaultForeground)
			.SetSelectedTextColor(DefaultForeground)
			);

		Set( "ContentBrowser.TileViewTooltip.ToolTipBorder", new FSlateColorBrush( FLinearColor::Black ) );
		Set( "ContentBrowser.TileViewTooltip.NonContentBorder", new BOX_BRUSH( "/Docking/TabContentArea", FMargin(4/16.0f) ) );
		Set( "ContentBrowser.TileViewTooltip.ContentBorder", new FSlateColorBrush( FStyleColors::Panel));
		Set( "ContentBrowser.TileViewTooltip.PillBorder", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 10.0f, FStyleColors::White, 1.0f));
		Set( "ContentBrowser.TileViewTooltip.UnsupportedAssetPillBorder", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 10.0f, FStyleColors::Warning, 1.0f));
		Set( "ContentBrowser.TileViewTooltip.NameFont", DEFAULT_FONT( "Regular", 12 ) );
		Set( "ContentBrowser.TileViewTooltip.AssetUserDescriptionFont", DEFAULT_FONT("Regular", 12 ) );

		// Columns view
		Set( "ContentBrowser.SortUp", new IMAGE_BRUSH( "Common/SortUpArrow", Icon8x4 ) );
		Set( "ContentBrowser.SortDown", new IMAGE_BRUSH( "Common/SortDownArrow", Icon8x4 ) );

		// Filter List - These are aliases for SBasicFilterBar styles in StarshipCoreStyle for backwards compatibility
		Set("ContentBrowser.FilterImage", new CORE_IMAGE_BRUSH_SVG("Starship/CoreWidgets/FilterBar/FilterColorSegment", FVector2f(8.f, 22.f)));
		Set("ContentBrowser.FilterBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, 3.0f));

		Set("ContentBrowser.FilterButton", FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FCheckBoxStyle>("FilterBar.FilterButton"));
		Set("ContentBrowser.FilterToolBar", FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("FilterBar.FilterToolBar"));

		// Sources view
		Set("ContentBrowser.Sources.Paths", new IMAGE_BRUSH("ContentBrowser/Sources_Paths_16x", Icon16x16));
		Set("ContentBrowser.Sources.Collections", new IMAGE_BRUSH("ContentBrowser/Sources_Collections_Standard_16x", Icon16x16));
		Set("ContentBrowser.Sources.Collections.Compact", new IMAGE_BRUSH("ContentBrowser/Sources_Collections_Compact_16x", Icon16x16));

		// Asset tags (common)
		Set("ContentBrowser.AssetTagBackground", new FSlateRoundedBoxBrush(FStyleColors::White, 2.0f));

		// Asset tags (standard)
		Set("ContentBrowser.AssetTagButton", FCheckBoxStyle()
			.SetUncheckedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2f(14.0f, 28.0f)))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2f(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUncheckedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2f(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUndeterminedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2f(14.0f, 28.0f)))
			.SetUndeterminedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2f(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUndeterminedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2f(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2f(14.0f, 28.0f)))
			.SetCheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2f(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat", FVector2f(14.0f, 28.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetPadding(0.0f)
			);

		Set("ContentBrowser.AssetTagNamePadding", FMargin(4.0f));
		Set("ContentBrowser.AssetTagCountPadding", FMargin(4.0f));

		// Asset tags (compact)
		Set("ContentBrowser.AssetTagButton.Compact", FCheckBoxStyle()
			.SetUncheckedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2f(10.0f, 20.0f)))
			.SetUncheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2f(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUncheckedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2f(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUndeterminedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2f(10.0f, 20.0f)))
			.SetUndeterminedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2f(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetUndeterminedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2f(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2f(10.0f, 20.0f)))
			.SetCheckedHoveredImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2f(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetCheckedPressedImage(IMAGE_BRUSH("ContentBrowser/AssetTagCheckbox_Flat_Compact", FVector2f(10.0f, 20.0f), FLinearColor(0.5f, 0.5f, 0.5f, 1.0f)))
			.SetPadding(0.0f)
			);

		Set("ContentBrowser.AssetTagNamePadding.Compact", FMargin(2.0f));
		Set("ContentBrowser.AssetTagCountPadding.Compact", FMargin(2.0f));

			
		Set( "ContentBrowser.PrimitiveCustom", new IMAGE_BRUSH( "ContentBrowser/ThumbnailCustom", Icon32x32 ) );
		Set( "ContentBrowser.PrimitiveSphere", new IMAGE_BRUSH_SVG("Icons/AssetIcons/ModelingSphereShaded_16", Icon16x16));
		Set( "ContentBrowser.PrimitiveCube", new IMAGE_BRUSH_SVG( "Icons/AssetIcons/ModelingBox_16", Icon16x16) );
		Set( "ContentBrowser.PrimitivePlane", new IMAGE_BRUSH_SVG( "Icons/AssetIcons/ModelingPlane_16", Icon16x16) );
		Set( "ContentBrowser.PrimitiveCylinder", new IMAGE_BRUSH_SVG("Icons/AssetIcons/ModelingCylinder_16", Icon16x16));
		Set( "ContentBrowser.ResetPrimitiveToDefault", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Undo", Icon20x20) );

		Set( "ContentBrowser.TopBar.Font", FTextBlockStyle( NormalText )
			.SetFont( DEFAULT_FONT( "Bold", 11 ) )
			.SetColorAndOpacity( FLinearColor( 1.0f, 1.0f, 1.0f ) )
			.SetHighlightColor( FLinearColor( 1.0f, 1.0f, 1.0f ) )
			.SetShadowOffset( FVector2f::UnitVector )
			.SetShadowColorAndOpacity( FLinearColor( 0.f, 0.f, 0.f, 0.9f ) ) );

		Set("ContentBrowser.ClassFont", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 7)));


		Set( "ContentBrowser.AddContent", new IMAGE_BRUSH_SVG( "Starship/ContentBrowser/ContentPack", Icon20x20 ) );
		Set( "ContentBrowser.ImportPackage", new IMAGE_BRUSH( "Icons/icon_Import_40x", Icon25x25 ) );
	
		// Asset Context Menu
		Set( "ContentBrowser.AssetActions", new CORE_IMAGE_BRUSH( "Icons/icon_tab_Tools_16x", Icon16x16 ) );
		Set( "ContentBrowser.AssetActions.Edit", new IMAGE_BRUSH( "Icons/Edit/icon_Edit_16x", Icon16x16 ) );
		Set( "ContentBrowser.AssetActions.OpenReadOnly", new IMAGE_BRUSH_SVG( "Starship/ContentBrowser/OpenReadOnly_16", Icon16x16 ) );
		Set( "ContentBrowser.AssetActions.Delete", new IMAGE_BRUSH( "Icons/icon_delete_16px", Icon16x16, FLinearColor( 0.4f, 0.5f, 0.7f, 1.0f ) ) );
		//Set( "ContentBrowser.AssetActions.Delete", new IMAGE_BRUSH( "Icons/Edit/icon_Edit_Delete_16x", Icon16x16) );
		Set( "ContentBrowser.AssetActions.Rename", new IMAGE_BRUSH( "Icons/Icon_Asset_Rename_16x", Icon16x16) );
		Set( "ContentBrowser.AssetActions.Duplicate", new IMAGE_BRUSH( "Icons/Edit/icon_Edit_Duplicate_16x", Icon16x16) );
		Set( "ContentBrowser.AssetActions.OpenSourceLocation", new IMAGE_BRUSH( "Icons/icon_Asset_Open_Source_Location_16x", Icon16x16) );
		Set( "ContentBrowser.AssetActions.OpenInExternalEditor", new IMAGE_BRUSH( "Icons/icon_Asset_Open_In_External_Editor_16x", Icon16x16) );
		Set( "ContentBrowser.AssetActions.PublicAssetToggle", new IMAGE_BRUSH("Icons/hiererchy_16x", Icon16x16));
		Set( "ContentBrowser.AssetActions.ReimportAsset", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/reimport", Icon16x16 ) );
		Set( "ContentBrowser.AssetActions.GoToCodeForAsset", new IMAGE_BRUSH( "GameProjectDialog/feature_code_32x", Icon16x16 ) );
		Set( "ContentBrowser.AssetActions.FindAssetInWorld", new IMAGE_BRUSH( "/Icons/icon_Genericfinder_16x", Icon16x16 ) );
		Set( "ContentBrowser.AssetActions.CreateThumbnail", new IMAGE_BRUSH( "Icons/icon_Asset_Create_Thumbnail_16x", Icon16x16) );
		Set( "ContentBrowser.AssetActions.DeleteThumbnail", new IMAGE_BRUSH( "Icons/icon_Asset_Delete_Thumbnail_16x", Icon16x16) );
		Set( "ContentBrowser.AssetActions.GenericFind", new IMAGE_BRUSH( "Icons/icon_Genericfinder_16x", Icon16x16) );
		Set( "ContentBrowser.AssetLocalization", new IMAGE_BRUSH( "Icons/icon_localization_16x", Icon16x16 ) );
		Set( "ContentBrowser.AssetActions.VolumeTexture", new IMAGE_BRUSH_SVG("Starship/AssetActions/volume-texture", Icon16x16));
		Set( "ContentBrowser.AssetActions.RemoveVertexColors", new IMAGE_BRUSH_SVG("Starship/AssetActions/remove-vertex-colors", Icon16x16));

		// ContentBrowser Commands Icons
		Set( "ContentBrowser.AssetViewCopyObjectPath", new IMAGE_BRUSH_SVG("../../Slate/Starship/Common/Copy", Icon16x16) );
		Set( "ContentBrowser.AssetViewCopyPackageName", new IMAGE_BRUSH_SVG("../../Slate/Starship/Common/Copy", Icon16x16) );

		Set( "MediaAsset.AssetActions.Play.Small", new IMAGE_BRUSH_SVG("Starship/Audio/PlayControlsPlay_16", Icon16x16));
		Set( "MediaAsset.AssetActions.Stop.Small", new IMAGE_BRUSH_SVG("Starship/Audio/PlayControlsStop_16", Icon16x16));
		Set( "MediaAsset.AssetActions.Pause.Small", new IMAGE_BRUSH("Icons/icon_SCueEd_Pause_16x", Icon16x16));
		Set( "MediaAsset.AssetActions.Solo.Small", new IMAGE_BRUSH_SVG("Starship/Audio/Solo_16", Icon16x16));
		Set( "MediaAsset.AssetActions.Mute.Small", new IMAGE_BRUSH_SVG("Starship/Audio/Mute_16", Icon16x16));
		Set( "MediaAsset.AssetActions.SoundConsolidation.Small", new IMAGE_BRUSH_SVG("Starship/Audio/SoundConsolidateAttenuation_16", Icon16x16));

		Set("MediaAsset.AssetActions.Play.Large", new IMAGE_BRUSH_SVG("Starship/Audio/PlayControlsPlay_16", Icon40x40));
		Set("MediaAsset.AssetActions.Stop.Large", new IMAGE_BRUSH_SVG("Starship/Audio/PlayControlsStop_16", Icon40x40));
		Set("MediaAsset.AssetActions.Pause.Large", new IMAGE_BRUSH("Icons/icon_SCueEd_Pause_40x", Icon40x40));
		Set("MediaAsset.AssetActions.Solo.Large", new IMAGE_BRUSH_SVG("Starship/Audio/Solo_16", Icon40x40));
		Set("MediaAsset.AssetActions.Mute.Large", new IMAGE_BRUSH_SVG("Starship/Audio/Mute_16", Icon40x40));
		Set( "MediaAsset.AssetActions.SoundConsolidation.Large", new IMAGE_BRUSH_SVG("Starship/Audio/SoundConsolidateAttenuation_16", Icon40x40));

		// Misc
		/** Should be moved, shared */ Set( "ContentBrowser.ThumbnailShadow", new BOX_BRUSH( "ContentBrowser/ThumbnailShadow" , FMargin( 4.0f / 64.0f ) ) );

		// Playback Action
		Set( "ContentBrowser.AssetAction.PlayIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/PlayIcon", Icon16x16));
		Set( "ContentBrowser.AssetAction.StopIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/StopIcon", Icon16x16));

		Set( "ContentBrowser.ColumnViewAssetIcon", new IMAGE_BRUSH( "Icons/doc_16x", Icon16x16 ) );

		Set( "ContentBrowser.ColumnViewFolderIcon", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/folder-closed", Icon16x16 ) );
		Set( "ContentBrowser.ColumnViewDeveloperFolderIcon", new IMAGE_BRUSH_SVG( "Starship/ContentBrowser/folder-developer", Icon16x16 ) );

		Set("ContentBrowser.ListViewFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/folder", Icon64x64));
		Set("ContentBrowser.ListViewVirtualFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/FolderLargeVirtual", Icon64x64));
		Set("ContentBrowser.ListViewVirtualFolderShadow", new IMAGE_BRUSH("Starship/ContentBrowser/FolderLargeVirtualShadow", FVector2f(256.f, 256.f)));
		Set("ContentBrowser.ListViewDeveloperFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/FolderLargeDeveloper", Icon64x64));
		Set("ContentBrowser.ListViewCodeFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/FolderLargeCode", Icon64x64));
		Set("ContentBrowser.ListViewPluginFolderIcon", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/FolderLargePlugin", Icon64x64));

		// Folder Tile Item Border
		Set("ContentBrowser.AssetTileItem.FolderAreaBackground", new FSlateRoundedBoxBrush(FStyleColors::Panel, 4.0f));
		Set("ContentBrowser.AssetTileItem.FolderAreaHoveredBackground", new FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f));
		Set("ContentBrowser.AssetTileItem.FolderAreaSelectedBackground", new FSlateRoundedBoxBrush(FStyleColors::Primary, 4.0f));
		Set("ContentBrowser.AssetTileItem.FolderAreaSelectedHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, 4.0f));

		Set("ContentBrowser.AssetTileItem.ThumbnailAreaBackground", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f));
		Set("ContentBrowser.AssetTileItem.NameAreaBackground", new FSlateRoundedBoxBrush(FStyleColors::Secondary, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.NameAreaHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::Hover, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.NameAreaSelectedBackground", new FSlateRoundedBoxBrush(FStyleColors::Primary, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.NameAreaSelectedHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.TinyFolderTextBorder", new FSlateRoundedBoxBrush(COLOR("#00000080"), 4.f, COLOR("#FFFFFF1A"), 1.f));

		// Asset Thumbnail Border
		Set("ContentBrowser.AssetTileItem.AssetBorderHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::Transparent, FVector4(4.0f, 4.0f, 4.0f, 4.0f), FStyleColors::Hover, 2.f));
		Set("ContentBrowser.AssetTileItem.AssetBorderSelectedBackground", new FSlateRoundedBoxBrush(FStyleColors::Transparent, FVector4(4.0f, 4.0f, 4.0f, 4.0f), FStyleColors::Primary, 2.f));
		Set("ContentBrowser.AssetTileItem.AssetBorderSelectedHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::Transparent, FVector4(4.0f, 4.0f, 4.0f, 4.0f), FStyleColors::PrimaryHover, 2.f));

		// Tile Item Border
		Set("ContentBrowser.AssetTileItem.AssetContent", new FSlateRoundedBoxBrush(FStyleColors::Secondary, FVector4(4.0f, 4.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.AssetContentHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::Hover, FVector4(4.0f, 4.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.AssetContentSelectedBackground", new FSlateRoundedBoxBrush(FStyleColors::Primary, FVector4(4.0f, 4.0f, 4.0f, 4.0f)));
		Set("ContentBrowser.AssetTileItem.AssetContentSelectedHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, FVector4(4.0f, 4.0f, 4.0f, 4.0f)));

		{
			FLinearColor OverlayColor = FStyleColors::Panel.GetSpecifiedColor();
            OverlayColor.A = 0.75f;
            Set("ContentBrowser.AssetTileItem.AssetThumbnailOverlayBorder", new FSlateRoundedBoxBrush(OverlayColor, 4.f));

			FLinearColor TransparentPrimary = FStyleColors::Primary.GetSpecifiedColor();
			TransparentPrimary.A = 0.0;
			Set("ContentBrowser.AssetTileItem.SelectedBorder", new FSlateRoundedBoxBrush(TransparentPrimary, 4.0f, FStyleColors::Primary, 1.0f));

			FLinearColor TransparentPrimaryHover = FStyleColors::PrimaryHover.GetSpecifiedColor();
			TransparentPrimaryHover.A = 0.0;
			Set("ContentBrowser.AssetTileItem.SelectedHoverBorder", new FSlateRoundedBoxBrush(TransparentPrimaryHover, 4.0f, FStyleColors::PrimaryHover, 1.0f));

			FLinearColor TransparentHover = FStyleColors::Hover.GetSpecifiedColor();
			TransparentHover.A = 0.0;
			Set("ContentBrowser.AssetTileItem.HoverBorder", new FSlateRoundedBoxBrush(TransparentHover, 4.0f, FStyleColors::Hover, 1.0f));
		}

		Set("ContentBrowser.AssetTileItem.DropShadow", new BOX_BRUSH("Starship/ContentBrowser/drop-shadow", FMargin(4.0f / 64.0f)));
		Set("ContentBrowser.FolderItem.DropShadow", new IMAGE_BRUSH("Starship/ContentBrowser/folder-drop-shadow", FVector2f(256.f, 256.f)));



		Set("ReferenceViewer.PathText", FEditableTextBoxStyle(NormalEditableTextBoxStyle)
			.SetFont(DEFAULT_FONT("Bold", 11)));

		Set( "ContentBrowser.ShowSourcesView", new IMAGE_BRUSH_SVG( "Starship/ContentBrowser/file-tree", Icon16x16 ) );
		Set( "ContentBrowser.HideSourcesView", new IMAGE_BRUSH_SVG( "Starship/ContentBrowser/file-tree-open", Icon16x16 ) );

		Set( "ContentBrowser.DirectoryUp", new IMAGE_BRUSH("Icons/icon_folder_up_16x", Icon16x16) );
		Set( "ContentBrowser.PathPickerButton", new IMAGE_BRUSH("Icons/ellipsis_12x", Icon12x12, FLinearColor::Black) );

		Set( "ContentBrowser.ContentDirty", new IMAGE_BRUSH_SVG("Starship/Common/DirtyBadge", Icon16x16) );
		Set( "ContentBrowser.AssetDragDropTooltipBackground", new BOX_BRUSH( "Old/Menu_Background", FMargin(8.0f/64.0f) ) );
		Set( "ContentBrowser.DragDropBackground", new FSlateRoundedBoxBrush(FStyleColors::Input, 0.f, FStyleColors::InputOutline, 1.f));
		Set( "ContentBrowser.DragDropAssetNumbersBorder", new FSlateRoundedBoxBrush(FStyleColors::Panel, 2.f));
		Set( "ContentBrowser.ThumbnailDragDropBackground", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 0.f, FStyleColors::InputOutline, 1.f));
		Set( "ContentBrowser.CollectionTreeDragDropBorder", new BOX_BRUSH( "Old/Window/ViewportDebugBorder", 0.8f ) );
		Set( "ContentBrowser.PopupMessageIcon", new IMAGE_BRUSH( "Icons/alert", Icon32x32) );
		Set( "ContentBrowser.NewFolderIcon", new CORE_IMAGE_BRUSH_SVG("Starship/Common/folder-plus", Icon16x16 ) );
		Set( "ContentBrowser.ShowInExplorer", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/show-in-explorer", Icon16x16));
		Set( "ContentBrowser.ReferenceViewer", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/reference-viewer", Icon16x16));
		Set( "ContentBrowser.SizeMap", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/size-map", Icon16x16));
		Set( "ContentBrowser.Collections", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/collections", Icon16x16));
		Set( "ContentBrowser.Migrate", new IMAGE_BRUSH_SVG("Starship/ContentBrowser/migrate", Icon16x16));
		Set( "ContentBrowser.Local", new IMAGE_BRUSH( "ContentBrowser/Content_Local_12x", Icon12x12 ) );
		Set( "ContentBrowser.Local.Small", new IMAGE_BRUSH( "ContentBrowser/Content_Local_16x", Icon16x16 ) );
		Set( "ContentBrowser.Local.Large", new IMAGE_BRUSH( "ContentBrowser/Content_Local_64x", Icon64x64 ) );
		Set( "ContentBrowser.Shared", new IMAGE_BRUSH( "ContentBrowser/Content_Shared_12x", Icon12x12 ) );
		Set( "ContentBrowser.Shared.Small", new IMAGE_BRUSH( "ContentBrowser/Content_Shared_16x", Icon16x16 ) );
		Set( "ContentBrowser.Shared.Large", new IMAGE_BRUSH( "ContentBrowser/Content_Shared_64x", Icon64x64 ) );
		Set( "ContentBrowser.Private", new IMAGE_BRUSH( "ContentBrowser/Content_Private_12x", Icon12x12 ) );
		Set( "ContentBrowser.Private.Small", new IMAGE_BRUSH( "ContentBrowser/Content_Private_16x", Icon16x16 ) );
		Set( "ContentBrowser.Private.Large", new IMAGE_BRUSH( "ContentBrowser/Content_Private_64x", Icon64x64 ) );
		Set( "ContentBrowser.CollectionStatus", new IMAGE_BRUSH( "/Icons/CollectionStatus_8x", Icon8x8 ) );


		Set( "AssetDiscoveryIndicator.MainStatusFont", DEFAULT_FONT( "Regular", 12 ) );
		Set( "AssetDiscoveryIndicator.SubStatusFont", DEFAULT_FONT( "Regular", 9 ) );

		Set( "ContentBrowser.SaveAllCurrentFolder", new IMAGE_BRUSH_SVG("Starship/Common/SaveCurrent", Icon16x16) );
		Set( "ContentBrowser.ResaveAllCurrentFolder", new IMAGE_BRUSH_SVG("Starship/Common/SaveCurrent", Icon16x16) );

		FToolBarStyle ContentBrowserToolBarStyle = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("SlimToolBar");

		ContentBrowserToolBarStyle.SetSeparatorBrush(FSlateNoResource());
		ContentBrowserToolBarStyle.SetSeparatorPadding(FMargin(4.0f, 0.0f));
		ContentBrowserToolBarStyle.SetBackgroundPadding(FMargin(4.f, 2.f, 0.f, 2.f));
	
		Set("ContentBrowser.ToolBar", ContentBrowserToolBarStyle);
	}
#endif // #if WITH_EDITOR
}

void FStarshipEditorStyle::FStyle::SetupLandscapeEditorStyle()
{
#if WITH_EDITOR
	// Landscape Editor
	{
		{
			// Tools
			Set("LandscapeEditor.NewLandscape",					new IMAGE_BRUSH("Icons/Landscape/Landscape_NewLandscape_x40",		Icon20x20));
			Set("LandscapeEditor.NewLandscape.Small",       	new IMAGE_BRUSH("Icons/Landscape/Landscape_NewLandscape_x40", 		Icon20x20));
			Set("LandscapeEditor.ResizeLandscape",				new IMAGE_BRUSH("Icons/Landscape/Landscape_Resize_x40",				Icon20x20));
			Set("LandscapeEditor.ResizeLandscape.Small",		new IMAGE_BRUSH("Icons/Landscape/Landscape_Resize_x40",				Icon20x20));
			Set("LandscapeEditor.ImportExportTool",				new IMAGE_BRUSH("Icons/Landscape/Landscape_ImportExport_x40",		Icon20x20));
			Set("LandscapeEditor.ImportExportTool.Small",		new IMAGE_BRUSH("Icons/Landscape/Landscape_ImportExport_x40",		Icon20x20));

			Set("LandscapeEditor.SculptTool",       			new IMAGE_BRUSH("Icons/Landscape/Landscape_Sculpt_x40",            Icon20x20));
			Set("LandscapeEditor.SculptTool.Small",       		new IMAGE_BRUSH("Icons/Landscape/Landscape_Sculpt_x40",            Icon20x20));
			Set("LandscapeEditor.EraseTool",					new IMAGE_BRUSH("Icons/Landscape/Landscape_Erase_x40",       	   Icon20x20));
			Set("LandscapeEditor.EraseTool.Small",				new IMAGE_BRUSH("Icons/Landscape/Landscape_Erase_x40",       	   Icon20x20));
			Set("LandscapeEditor.PaintTool",        			new IMAGE_BRUSH("Icons/Landscape/Landscape_PaintTool_x40",         Icon20x20));
			Set("LandscapeEditor.PaintTool.Small",        		new IMAGE_BRUSH("Icons/Landscape/Landscape_PaintTool_x40",         Icon20x20));
			Set("LandscapeEditor.SmoothTool",       			new IMAGE_BRUSH("Icons/Landscape/Landscape_Smooth_x40",            Icon20x20));
			Set("LandscapeEditor.SmoothTool.Small",       		new IMAGE_BRUSH("Icons/Landscape/Landscape_Smooth_x40",            Icon20x20));
			Set("LandscapeEditor.FlattenTool",      			new IMAGE_BRUSH("Icons/Landscape/Landscape_Flatten_x40",           Icon20x20));
			Set("LandscapeEditor.FlattenTool.Small",      		new IMAGE_BRUSH("Icons/Landscape/Landscape_Flatten_x40",           Icon20x20));
			Set("LandscapeEditor.RampTool",         			new IMAGE_BRUSH("Icons/Landscape/Landscape_Ramp_x40",              Icon20x20));
			Set("LandscapeEditor.RampTool.Small",         		new IMAGE_BRUSH("Icons/Landscape/Landscape_Ramp_x40",              Icon20x20));
			Set("LandscapeEditor.ErosionTool",      			new IMAGE_BRUSH("Icons/Landscape/Landscape_Erosion_x40",           Icon20x20));
			Set("LandscapeEditor.ErosionTool.Small",      		new IMAGE_BRUSH("Icons/Landscape/Landscape_Erosion_x40",           Icon20x20));
			Set("LandscapeEditor.HydroErosionTool", 			new IMAGE_BRUSH("Icons/Landscape/Landscape_HydroErosion_x40",      Icon20x20));
			Set("LandscapeEditor.HydroErosionTool.Small", 		new IMAGE_BRUSH("Icons/Landscape/Landscape_HydroErosion_x40",      Icon20x20));
			Set("LandscapeEditor.NoiseTool",        			new IMAGE_BRUSH("Icons/Landscape/Landscape_Noise_x40",             Icon20x20));
			Set("LandscapeEditor.NoiseTool.Small",        		new IMAGE_BRUSH("Icons/Landscape/Landscape_Noise_x40",             Icon20x20));
			Set("LandscapeEditor.VisibilityTool",   			new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_VisibilityTool_x40",  Icon20x20));
			Set("LandscapeEditor.VisibilityTool.Small",   		new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_VisibilityTool_x40",  Icon20x20));
			Set("LandscapeEditor.BlueprintBrushTool", 			new IMAGE_BRUSH("Icons/Landscape/Landscape_BlueprintTool_x40",     Icon20x20));
			Set("LandscapeEditor.BlueprintBrushTool.Small", 	new IMAGE_BRUSH("Icons/Landscape/Landscape_BlueprintTool_x40",     Icon20x20));

			Set("LandscapeEditor.SelectComponentTool", 			new IMAGE_BRUSH("Icons/Landscape/Landscape_ComponentSelect_x40",   Icon20x20));
			Set("LandscapeEditor.SelectComponentTool.Small", 	new IMAGE_BRUSH("Icons/Landscape/Landscape_ComponentSelect_x40",   Icon20x20));
			Set("LandscapeEditor.AddComponentTool",    			new IMAGE_BRUSH("Icons/Landscape/Landscape_ComponentAdd_x40",      Icon20x20));
			Set("LandscapeEditor.AddComponentTool.Small",    	new IMAGE_BRUSH("Icons/Landscape/Landscape_ComponentAdd_x40",      Icon20x20));
			Set("LandscapeEditor.DeleteComponentTool", 			new IMAGE_BRUSH("Icons/Landscape/Landscape_ComponentDelete_x40",   Icon20x20));
			Set("LandscapeEditor.DeleteComponentTool.Small", 	new IMAGE_BRUSH("Icons/Landscape/Landscape_ComponentDelete_x40",   Icon20x20));
			Set("LandscapeEditor.MoveToLevelTool",     			new IMAGE_BRUSH("Icons/Landscape/Landscape_ComponentMove_x40",     Icon20x20));
			Set("LandscapeEditor.MoveToLevelTool.Small",     	new IMAGE_BRUSH("Icons/Landscape/Landscape_ComponentMove_x40",     Icon20x20));

			Set("LandscapeEditor.RegionSelectTool",    			new IMAGE_BRUSH("Icons/Landscape/Landscape_RegionSelect_x40",      Icon20x20));
			Set("LandscapeEditor.RegionSelectTool.Small",    	new IMAGE_BRUSH("Icons/Landscape/Landscape_RegionSelect_x40",      Icon20x20));
			Set("LandscapeEditor.RegionCopyPasteTool",			new IMAGE_BRUSH("Icons/Landscape/Landscape_CopyPasteTool_x40",     Icon20x20));
			Set("LandscapeEditor.RegionCopyPasteTool.Small",	new IMAGE_BRUSH("Icons/Landscape/Landscape_CopyPasteTool_x40",     Icon20x20));

			Set("LandscapeEditor.MirrorTool",       			new IMAGE_BRUSH("Icons/Landscape/Landscape_Mirror_x40",            Icon20x20));
			Set("LandscapeEditor.MirrorTool.Small",       		new IMAGE_BRUSH("Icons/Landscape/Landscape_Mirror_x40",            Icon20x20));

			Set("LandscapeEditor.SplineTool",       			new IMAGE_BRUSH("Icons/Landscape/Landscape_EditSplines_x40",       Icon20x20));
			Set("LandscapeEditor.SplineTool.Small",       		new IMAGE_BRUSH("Icons/Landscape/Landscape_EditSplines_x40",       Icon20x20));

			// Brush Sets
			Set("LandscapeEditor.CircleBrush",        			new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushCircle_x40",       Icon20x20));
			Set("LandscapeEditor.CircleBrush.Small",        	new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushCircle_x40",       Icon20x20));
			Set("LandscapeEditor.AlphaBrush",         			new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushAlpha_x40",        Icon20x20));
			Set("LandscapeEditor.AlphaBrush.Small",         	new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushAlpha_x40",        Icon20x20));
			Set("LandscapeEditor.AlphaBrush_Pattern", 			new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushPattern_x40",      Icon20x20));
			Set("LandscapeEditor.AlphaBrush_Pattern.Small", 	new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushPattern_x40",      Icon20x20));
			Set("LandscapeEditor.ComponentBrush",     			new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushComponent_x40",    Icon20x20));
			Set("LandscapeEditor.ComponentBrush.Small",     	new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushComponent_x40",    Icon20x20));
			Set("LandscapeEditor.GizmoBrush",         			new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushGizmo_x40",        Icon20x20));
			Set("LandscapeEditor.GizmoBrush.Small",         	new IMAGE_BRUSH("Icons/Landscape/Landscape_BrushGizmo_x40",        Icon20x20));

			// Brushes
			Set("LandscapeEditor.CircleBrush_Smooth",   		new IMAGE_BRUSH("Icons/Landscape/Landscape_FalloffSmooth_x40",     Icon20x20));
			Set("LandscapeEditor.CircleBrush_Smooth.Small",   	new IMAGE_BRUSH("Icons/Landscape/Landscape_FalloffSmooth_x40",     Icon20x20));
			Set("LandscapeEditor.CircleBrush_Linear",   		new IMAGE_BRUSH("Icons/Landscape/Landscape_FalloffLinear_x40",     Icon20x20));
			Set("LandscapeEditor.CircleBrush_Linear.Small",   	new IMAGE_BRUSH("Icons/Landscape/Landscape_FalloffLinear_x40",     Icon20x20));
			Set("LandscapeEditor.CircleBrush_Spherical",		new IMAGE_BRUSH("Icons/Landscape/Landscape_FalloffSpherical_x40",  Icon20x20));
			Set("LandscapeEditor.CircleBrush_Spherical.Small",	new IMAGE_BRUSH("Icons/Landscape/Landscape_FalloffSpherical_x40",  Icon20x20));
			Set("LandscapeEditor.CircleBrush_Tip",      		new IMAGE_BRUSH("Icons/Landscape/Landscape_FalloffTip_x40",        Icon20x20));
			Set("LandscapeEditor.CircleBrush_Tip.Small",      	new IMAGE_BRUSH("Icons/Landscape/Landscape_FalloffTip_x40",        Icon20x20));
		}

		Set("LandscapeEditor.Brush.AffectsHeightmap", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_Heightmap_16x", Icon16x16));
		Set("LandscapeEditor.Brush.AffectsHeightmap.Disabled", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_Heightmap_Disabled_16x", Icon16x16));

		Set("LandscapeEditor.Brush.AffectsHeight.Enabled", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_Height_Enabled_16x", Icon16x16));
		Set("LandscapeEditor.Brush.AffectsHeight.Disabled", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_Height_Disabled_16x", Icon16x16));
		
		Set("LandscapeEditor.Brush.AffectsWeightmap", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_Weightmap_16x", Icon16x16));
		Set("LandscapeEditor.Brush.AffectsWeightmap.Disabled", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_Weightmap_Disabled_16x", Icon16x16));

		Set("LandscapeEditor.Brush.AffectsWeight.Enabled", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_Weight_Enabled_16x", Icon16x16));
		Set("LandscapeEditor.Brush.AffectsWeight.Disabled", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_Weight_Disabled_16x", Icon16x16));
		
		Set("LandscapeEditor.Brush.AffectsVisibilityLayer.Enabled", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_VisibilityLayer_Enabled_16x", Icon16x16));
		Set("LandscapeEditor.Brush.AffectsVisibilityLayer.Disabled", new IMAGE_BRUSH("Icons/icon_Landscape_Affects_VisibilityLayer_Disabled_16x", Icon16x16));

		Set("LandscapeEditor.Brush.Action_Regular", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Brush_Action_Regular_16x", Icon16x16));
		Set("LandscapeEditor.Brush.Action_Inverse", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Brush_Action_Inverse_16x", Icon16x16));

		Set("LandscapeEditor.InspectedObjects.ShowDetails", new CORE_IMAGE_BRUSH_SVG("Starship/Common/settings", Icon16x16));

		// Target List
		Set("LandscapeEditor.TargetList.RowBackground",        new FSlateNoResource());
		Set("LandscapeEditor.TargetList.RowBackgroundHovered", new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FStyleColors::SelectHover));
		Set("LandscapeEditor.TargetList.RowSelected",          new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FStyleColors::Select));
		Set("LandscapeEditor.TargetList.RowSelectedHovered",   new BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FStyleColors::PrimaryHover));

		Set("LandscapeEditor.Target_Heightmap",  new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_Heightmap_64x",  Icon48x48));
		Set("LandscapeEditor.Target_Weightmap",	 new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_Weightmap_64x",  Icon48x48));
		Set("LandscapeEditor.Target_Visibility", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_Visibility_64x", Icon48x48));
		Set("LandscapeEditor.Target_Unknown",    new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_Unknown_64x",    Icon48x48));

		Set("LandscapeEditor.Target_DisplayOrder.Default.Ascending", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_DisplayOrder_Default_Ascending_16x", Icon16x16));
		Set("LandscapeEditor.Target_DisplayOrder.Default.Descending", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_DisplayOrder_Default_Descending_16x", Icon16x16));
		Set("LandscapeEditor.Target_DisplayOrder.Alphabetical.Ascending", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_DisplayOrder_Alphabetical_Ascending_16x", Icon16x16));
		Set("LandscapeEditor.Target_DisplayOrder.Alphabetical.Descending", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_DisplayOrder_Alphabetical_Descending_16x", Icon16x16));
		Set("LandscapeEditor.Target_DisplayOrder.ByBlendMethod.Ascending", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_DisplayOrder_ByBlendMethod_Ascending_16x", Icon16x16));
		Set("LandscapeEditor.Target_DisplayOrder.ByBlendMethod.Descending", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_DisplayOrder_ByBlendMethod_Descending_16x", Icon16x16));
		Set("LandscapeEditor.Target_DisplayOrder.Custom", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Target_DisplayOrder_Custom_16x", Icon16x16));

		Set("LandscapeEditor.TargetList.DropZone.Above", new BOX_BRUSH("Common/VerticalBoxDropZoneIndicator_Above", FMargin(10.0f / 16.0f, 10.0f / 16.0f, 0, 0), SelectionColor_Subdued));
		Set("LandscapeEditor.TargetList.DropZone.Below", new BOX_BRUSH("Common/VerticalBoxDropZoneIndicator_Below", FMargin(10.0f / 16.0f, 0, 0, 10.0f / 16.0f), SelectionColor_Subdued));

		Set("LandscapeEditor.Layer.Sync", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Layers_From_Material_16x", Icon16x16));
		Set("LandscapeEditor.Layer.AutoFill", new IMAGE_BRUSH_SVG("Icons/Landscape/Landscape_Layers_AutoFill_16x", Icon16x16));
}

#endif
}

void FStarshipEditorStyle::FStyle::SetupToolkitStyles()
{
#if WITH_EDITOR
	// Project Browser
	{
		Set("ProjectBrowser.VersionOverlayText", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 14))
			.SetShadowOffset(FVector2f(0.f, 1.f)));


		const FTableRowStyle ProjectBrowserTableRowStyle = FTableRowStyle()
			.SetEvenRowBackgroundBrush(FSlateNoResource())
			.SetEvenRowBackgroundHoveredBrush(FSlateNoResource())
			.SetOddRowBackgroundBrush(FSlateNoResource())
			.SetOddRowBackgroundHoveredBrush(FSlateNoResource())
			.SetSelectorFocusedBrush(FSlateNoResource())
			.SetActiveBrush(FSlateNoResource())
			.SetActiveHoveredBrush(FSlateNoResource())
			.SetInactiveBrush(FSlateNoResource())
			.SetInactiveHoveredBrush(FSlateNoResource())
			.SetActiveHighlightedBrush(FSlateNoResource())
			.SetInactiveHighlightedBrush(FSlateNoResource())
			.SetTextColor(FStyleColors::Foreground)
			.SetSelectedTextColor(FStyleColors::ForegroundInverted);

		Set("ProjectBrowser.TableRow", ProjectBrowserTableRowStyle);

		Set("ProjectBrowser.MajorCategory.ViewBorder", new FSlateRoundedBoxBrush(FStyleColors::Recessed, 4.0f));
		Set("ProjectBrowser.MajorCategory.Size", FVector2D(304, 104));
		Set("ProjectBrowser.MajorCategory.FallBackThumbnail", new IMAGE_BRUSH("Starship/Projects/CustomTemplate_2x", GetVector("ProjectBrowser.MajorCategory.Size")));

		Set("ProjectBrowser.TileViewTooltip.ToolTipBorder", new FSlateColorBrush(FLinearColor::Black));
		Set("ProjectBrowser.TileViewTooltip.NonContentBorder", new BOX_BRUSH("/Docking/TabContentArea", FMargin(4 / 16.0f)));
		Set("ProjectBrowser.TileViewTooltip.ContentBorder", new BOX_BRUSH("Common/GroupBorder", FMargin(4.0f / 16.0f)));
		Set("ProjectBrowser.TileViewTooltip.NameFont", DEFAULT_FONT("Regular", 12));

		Set("ProjectBrowser.ProjectTile.Font", DEFAULT_FONT("Regular", 9));
		Set("ProjectBrowser.ProjectTile.ThumbnailAreaBackground", new FSlateRoundedBoxBrush(COLOR("#474747FF"), FVector4(4.0f,4.0f,0.0f,0.0f)));
		Set("ProjectBrowser.ProjectTile.NameAreaBackground", new FSlateRoundedBoxBrush(EStyleColor::Header, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ProjectBrowser.ProjectTile.NameAreaHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::Hover, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ProjectBrowser.ProjectTile.NameAreaSelectedBackground", new FSlateRoundedBoxBrush(FStyleColors::Primary, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ProjectBrowser.ProjectTile.NameAreaSelectedHoverBackground", new FSlateRoundedBoxBrush(FStyleColors::PrimaryHover, FVector4(0.0f, 0.0f, 4.0f, 4.0f)));
		Set("ProjectBrowser.ProjectTile.DropShadow", new BOX_BRUSH("Starship/ContentBrowser/drop-shadow", FMargin(4.0f / 64.0f)));

		{
			FLinearColor TransparentPrimary = FStyleColors::Primary.GetSpecifiedColor();
			TransparentPrimary.A = 0.0;
			Set("ProjectBrowser.ProjectTile.SelectedBorder", new FSlateRoundedBoxBrush(TransparentPrimary, 4.0f, FStyleColors::Primary, 1.0f));

			FLinearColor TransparentPrimaryHover = FStyleColors::PrimaryHover.GetSpecifiedColor();
			TransparentPrimaryHover.A = 0.0;
			Set("ProjectBrowser.ProjectTile.SelectedHoverBorder", new FSlateRoundedBoxBrush(TransparentPrimaryHover, 4.0f, FStyleColors::PrimaryHover, 1.0f));

			FLinearColor TransparentHover = FStyleColors::Hover.GetSpecifiedColor();
			TransparentHover.A = 0.0;
			Set("ProjectBrowser.ProjectTile.HoverBorder", new FSlateRoundedBoxBrush(TransparentHover, 4.0f, FStyleColors::Hover, 1.0f));
		}
	}

	// Toolkit Display
	{
		Set("ToolkitDisplay.UnsavedChangeIcon", new IMAGE_BRUSH("Common/UnsavedChange", Icon8x8));
		Set("ToolkitDisplay.MenuDropdown", new IMAGE_BRUSH("Common/ComboArrow", Icon8x8));
		Set("ToolkitDisplay.ColorOverlay", new BOX_BRUSH("/Docking/Tab_ColorOverlay", 4 / 16.0f));

		FComboButtonStyle ComboButton = FComboButtonStyle()
			.SetButtonStyle(Button)
			.SetDownArrowImage(IMAGE_BRUSH("Common/ComboArrow", Icon8x8))
			// Multiboxes draw their own border so we don't want a default content border
			.SetMenuBorderBrush(FSlateNoResource())
			.SetMenuBorderPadding(FMargin(0.0f));
		Set("ToolkitDisplay.ComboButton", ComboButton);
	}

	// Generic Editor
	{
		Set( "GenericEditor.Tabs.Properties", new IMAGE_BRUSH( "/Icons/icon_tab_SelectionDetails_16x", Icon16x16 ) );
	}

	// CurveTable Editor
	{
		Set( "CurveTableEditor.Tabs.Properties", new IMAGE_BRUSH( "/Icons/icon_tab_SelectionDetails_16x", Icon16x16 ) );
		Set( "CurveTableEditor.CurveView", new IMAGE_BRUSH("GenericCurveEditor/Icons/GenericCurveEditor_48x", Icon20x20));
		Set( "CurveTableEditor.TableView", new CORE_IMAGE_BRUSH_SVG("Starship/Common/layout-spreadsheet",  Icon20x20));

		// SEditableTextBox defaults...
		Set("CurveTableEditor.Cell.Text", FEditableTextBoxStyle()
			.SetTextStyle(NormalText)
			.SetBackgroundImageNormal(FSlateNoResource())
			.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Input, 0.0f, FStyleColors::Hover, 1.0f))
			.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Input, 0.0f, FStyleColors::Primary, 1.0f))
			.SetBackgroundImageReadOnly(FSlateNoResource())
			.SetPadding(FMargin(4.0f))
			.SetForegroundColor(FStyleColors::Foreground)
			.SetBackgroundColor(FStyleColors::White)
			.SetReadOnlyForegroundColor(FSlateColor::UseForeground())
			.SetFocusedForegroundColor(FStyleColors::White)
		);


	}

	// DataTable Editor
	{
		Set( "DataTableEditor.Tabs.Properties", new IMAGE_BRUSH( "/Icons/icon_tab_SelectionDetails_16x", Icon16x16 ) );

		Set("DataTableEditor.Copy", new IMAGE_BRUSH("/Icons/Edit/icon_Edit_Copy_40x", Icon32x32));
		Set("DataTableEditor.Paste", new IMAGE_BRUSH("/Icons/Edit/icon_Edit_Paste_40x", Icon32x32));
		Set("DataTableEditor.Duplicate", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Duplicate_40x", Icon32x32));
		Set("DataTableEditor.Save", new IMAGE_BRUSH("Icons/icon_SaveAsset_40x", Icon16x16));
		Set("DataTableEditor.Browse", new IMAGE_BRUSH("Icons/lens_12x", Icon16x16));
		Set("DataTableEditor.Add", new IMAGE_BRUSH("Icons/icon_add_40x", Icon32x32));
		Set("DataTableEditor.Remove", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Delete_40x", Icon32x32));

		Set("DataTableEditor.Copy.Small", new IMAGE_BRUSH("/Icons/Edit/icon_Edit_Copy_40x", Icon16x16));
		Set("DataTableEditor.Paste.Small", new IMAGE_BRUSH("/Icons/Edit/icon_Edit_Paste_40x", Icon16x16));
		Set("DataTableEditor.Duplicate.Small", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Duplicate_40x", Icon16x16));
		Set("DataTableEditor.Add.Small", new IMAGE_BRUSH("Icons/icon_add_40x", Icon16x16));
		Set("DataTableEditor.Remove.Small", new IMAGE_BRUSH("Icons/Edit/icon_Edit_Delete_40x", Icon16x16));



		Set( "DataTableEditor.CellText", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT("Regular", 9 ))
			);

		Set( "DataTableEditor.NameListViewRow", FTableRowStyle(NormalTableRowStyle)
			.SetEvenRowBackgroundBrush( BOX_BRUSH( "Common/TableViewMajorColumn", 4.f/32.f ) )
			.SetEvenRowBackgroundHoveredBrush( BOX_BRUSH( "Common/TableViewMajorColumn", 4.f/32.f ) )
			.SetOddRowBackgroundBrush( BOX_BRUSH( "Common/TableViewMajorColumn", 4.f/32.f ) )
			.SetOddRowBackgroundHoveredBrush( BOX_BRUSH( "Common/TableViewMajorColumn", 4.f/32.f ) )
			.SetSelectorFocusedBrush( FSlateNoResource() )
			.SetActiveBrush( BOX_BRUSH( "Common/TableViewMajorColumn", 4.f/32.f ) )
			.SetActiveHoveredBrush( BOX_BRUSH( "Common/TableViewMajorColumn", 4.f/32.f ) )
			.SetInactiveBrush( BOX_BRUSH( "Common/TableViewMajorColumn", 4.f/32.f ) )
			.SetInactiveHoveredBrush( BOX_BRUSH( "Common/TableViewMajorColumn", 4.f/32.f ) )
			.SetTextColor( DefaultForeground )
			.SetSelectedTextColor( DefaultForeground )
		);

		Set("DataTableEditor.CellListViewRow", FTableRowStyle(NormalTableRowStyle)
			.SetEvenRowBackgroundBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f)))
			.SetEvenRowBackgroundHoveredBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle_Hovered", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f)))
			.SetOddRowBackgroundBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle", Icon16x16, FLinearColor(0.2f, 0.2f, 0.2f)))
			.SetOddRowBackgroundHoveredBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle_Hovered", Icon16x16, FLinearColor(0.2f, 0.2f, 0.2f)))
			.SetActiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.075f, 0.075f, 0.075f)))
			.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.075f, 0.075f, 0.075f)))
			.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.075f, 0.075f, 0.075f)))
			.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.075f, 0.075f, 0.075f)))
			.SetTextColor(DefaultForeground)
			.SetSelectedTextColor(DefaultForeground)
		);

		Set("DataTableEditor.DragDropObject", new BOX_BRUSH("Common/TextBox_Special_Active", FMargin(8.0f / 32.0f)));
		Set("DataTableEditor.DragDropHoveredTarget", new BOX_BRUSH("Common/TextBox_Special_Active", FMargin(8.0f / 32.0f), SelectionColor_Pressed));

	}

	// StringTable Editor
	{
		Set("StringTableEditor.Tabs.Properties", new IMAGE_BRUSH("/Icons/icon_tab_SelectionDetails_16x", Icon16x16));
	}
#endif //#if WITH_EDITOR

	// Material Editor
#if WITH_EDITOR
	{
		Set( "MaterialEditor.Tabs.HLSLCode", new IMAGE_BRUSH( "/Icons/icon_MatEd_HLSL_Code_16x", Icon16x16 ) );

		Set( "MaterialEditor.Layers.EditableFont", DEFAULT_FONT("Regular", 8));
		Set("MaterialEditor.Layers.EditableFontImportant", DEFAULT_FONT("Bold", FStarshipCoreStyle::RegularTextSize));
		Set( "MaterialEditor.NormalFont", DEFAULT_FONT( "Regular", 9 ) );
		Set( "MaterialEditor.BoldFont", DEFAULT_FONT( "Bold", 9 ) );

		Set( "MaterialEditor.Apply", new IMAGE_BRUSH_SVG( "Starship/Common/Apply", Icon20x20 ) );
		Set( "MaterialEditor.LiveUpdate", new IMAGE_BRUSH_SVG("Starship/MaterialEditor/LiveUpdate", Icon20x20));
		Set( "MaterialEditor.Hierarchy", new IMAGE_BRUSH_SVG("Starship/MaterialEditor/Hierarchy", Icon20x20));

		Set( "MaterialEditor.SetCylinderPreview", new IMAGE_BRUSH_SVG("Icons/AssetIcons/ModelingCylinder_16", Icon16x16));
		Set( "MaterialEditor.SetSpherePreview", new IMAGE_BRUSH_SVG("Icons/AssetIcons/ModelingSphereShaded_16", Icon16x16));
		Set( "MaterialEditor.SetShaderBallPreview", new IMAGE_BRUSH_SVG("Icons/AssetIcons/ModelingShaderBall_16", Icon16x16));
		Set( "MaterialEditor.SetPlanePreview", new IMAGE_BRUSH_SVG( "Icons/AssetIcons/ModelingPlane_16", Icon16x16) );
		Set( "MaterialEditor.SetCubePreview", new IMAGE_BRUSH_SVG( "Icons/AssetIcons/ModelingBox_16", Icon16x16) );
		Set( "MaterialEditor.SetPreviewMeshFromSelection", new IMAGE_BRUSH_SVG( "Starship/AssetIcons/StaticMesh_16", Icon16x16 ) );
		Set( "MaterialEditor.TogglePreviewGrid", new IMAGE_BRUSH_SVG( "Starship/Common/Grid", Icon16x16 ) );
		Set( "MaterialEditor.ToggleMaterialStats", new IMAGE_BRUSH_SVG( "Starship/Common/MaterialAnalyzer", Icon20x20 ) );
		Set( "MaterialEditor.ToggleMaterialStats.Tab", new IMAGE_BRUSH_SVG("Starship/Common/MaterialAnalyzer", Icon16x16));
		Set( "MaterialEditor.TogglePlatformStats", new IMAGE_BRUSH_SVG( "Starship/MaterialEditor/PlatformStats", Icon20x20 ) );
		Set("MaterialEditor.TogglePlatformStats.Tab", new IMAGE_BRUSH_SVG("Starship/MaterialEditor/PlatformStats", Icon16x16));
		Set( "MaterialEditor.CameraHome", new IMAGE_BRUSH_SVG( "Starship/Common/Home", Icon20x20 ) );
		Set( "MaterialEditor.FindInMaterial", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/Search", Icon20x20 ) );


		Set("MaterialEditor.CellListViewRow", FTableRowStyle(NormalTableRowStyle)
			.SetEvenRowBackgroundBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f)))
			.SetEvenRowBackgroundHoveredBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle_Hovered", Icon16x16, FLinearColor(0.5f, 0.5f, 0.5f)))
			.SetOddRowBackgroundBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle", Icon16x16, FLinearColor(0.35f, 0.35f, 0.35f)))
			.SetOddRowBackgroundHoveredBrush(IMAGE_BRUSH("PropertyView/DetailCategoryMiddle_Hovered", Icon16x16, FLinearColor(0.35f, 0.35f, 0.35f)))
			.SetActiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.075f, 0.075f, 0.075f)))
			.SetActiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.075f, 0.075f, 0.075f)))
			.SetInactiveBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.075f, 0.075f, 0.075f)))
			.SetInactiveHoveredBrush(IMAGE_BRUSH("Common/Selection", Icon8x8, FLinearColor(0.075f, 0.075f, 0.075f)))
			.SetTextColor(DefaultForeground)
			.SetSelectedTextColor(DefaultForeground)
		);
	}

	// Material Instance Editor
	{
		Set( "MaterialInstanceEditor.Tabs.Properties", new IMAGE_BRUSH( "/Icons/icon_tab_SelectionDetails_16x", Icon16x16 ) );
		Set("MaterialEditor.ShowAllMaterialParameters", new IMAGE_BRUSH_SVG("Starship/GraphEditors/HideUnrelated", Icon20x20));
	}
	
	// Sound Class Editor
	{
		Set( "SoundClassEditor.Tabs.Properties", new IMAGE_BRUSH( "/Icons/icon_tab_SelectionDetails_16x", Icon16x16 ) );

		Set("SoundClassEditor.ToggleSolo", new IMAGE_BRUSH("Icons/icon_SCueEd_Solo_40x", Icon40x40));
		Set("SoundClassEditor.ToggleSolo.Small", new IMAGE_BRUSH("Icons/icon_SCueEd_Solo_40x", Icon20x20));
		Set("SoundClassEditor.ToggleMute", new IMAGE_BRUSH("Icons/icon_SCueEd_Mute_40x", Icon40x40));
		Set("SoundClassEditor.ToggleMute.Small", new IMAGE_BRUSH("Icons/icon_SCueEd_Mute_40x", Icon20x20));
	}

	// Font Editor
	{
		// Tab icons
		{
			Set( "FontEditor.Tabs.Preview", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/Search", Icon16x16 ) );
			Set( "FontEditor.Tabs.Properties", new IMAGE_BRUSH( "/Icons/icon_tab_SelectionDetails_16x", Icon16x16 ) );
			Set( "FontEditor.Tabs.PageProperties", new IMAGE_BRUSH( "/Icons/properties_16x", Icon16x16 ) );
		}

		Set( "FontEditor.Update", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/Update", Icon20x20 ) );
		Set( "FontEditor.UpdateAll", new IMAGE_BRUSH_SVG( "Starship/Common/UpdateAll", Icon20x20 ) );
		Set( "FontEditor.ExportPage", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/export_20", Icon20x20 ) );
		Set( "FontEditor.ExportAllPages", new IMAGE_BRUSH_SVG( "Starship/Common/ExportAll", Icon20x20 ) );
		Set( "FontEditor.FontBackgroundColor", new IMAGE_BRUSH_SVG( "Starship/FontEditor/FontBackground", Icon20x20 ) );
		Set( "FontEditor.FontForegroundColor", new IMAGE_BRUSH_SVG( "Starship/FontEditor/FontForeground", Icon20x20 ) );

		Set( "FontEditor.Button_Add", new IMAGE_BRUSH( "Icons/PlusSymbol_12x", Icon12x12 ) );
		Set( "FontEditor.Button_Delete", new IMAGE_BRUSH("Icons/Cross_12x", Icon12x12 ) );
	}

	// SoundCueGraph Editor
	{
		Set( "SoundCueGraphEditor.PlayCue", new IMAGE_BRUSH( "Icons/icon_SCueEd_PlayCue_40x", Icon40x40 ) );
		Set( "SoundCueGraphEditor.PlayCue.Small", new IMAGE_BRUSH( "Icons/icon_SCueEd_PlayCue_40x", Icon20x20 ) );
		Set( "SoundCueGraphEditor.PlayNode", new IMAGE_BRUSH( "Icons/icon_SCueEd_PlayNode_40x", Icon40x40 ) );
		Set( "SoundCueGraphEditor.PlayNode.Small", new IMAGE_BRUSH( "Icons/icon_SCueEd_PlayNode_40x", Icon20x20 ) );
		Set( "SoundCueGraphEditor.StopCueNode", new IMAGE_BRUSH( "Icons/icon_SCueEd_Stop_40x", Icon40x40 ) );
		Set( "SoundCueGraphEditor.StopCueNode.Small", new IMAGE_BRUSH( "Icons/icon_SCueEd_Stop_40x", Icon20x20 ) );

		Set("SoundCueGraphEditor.ToggleSolo", new IMAGE_BRUSH("Icons/icon_SCueEd_Solo_40x", Icon40x40));
		Set("SoundCueGraphEditor.ToggleSolo.Small", new IMAGE_BRUSH("Icons/icon_SCueEd_Solo_40x", Icon20x20));
		Set("SoundCueGraphEditor.ToggleMute", new IMAGE_BRUSH("Icons/icon_SCueEd_Mute_40x", Icon40x40));
		Set("SoundCueGraphEditor.ToggleMute.Small", new IMAGE_BRUSH("Icons/icon_SCueEd_Mute_40x", Icon20x20));
	}

	// Static Mesh Editor
	{
		Set("StaticMeshEditor.Tabs.Properties", new IMAGE_BRUSH_SVG("Starship/Common/Details", Icon16x16));
		Set("StaticMeshEditor.Tabs.SocketManager", new IMAGE_BRUSH_SVG("Starship/StaticMeshEditor/SocketManager", Icon16x16));
		Set("StaticMeshEditor.Tabs.ConvexDecomposition", new IMAGE_BRUSH_SVG("Starship/StaticMeshEditor/ConvexDecomposition", Icon16x16));
		Set("StaticMeshEditor.SetShowWireframe", new IMAGE_BRUSH_SVG("Starship/Common/BrushWireframe", Icon16x16));
		Set("StaticMeshEditor.SetShowVertexColor", new IMAGE_BRUSH_SVG( "Starship/Common/SetShowVertexColors", Icon16x16));
		Set("StaticMeshEditor.SetRealtimePreview", new IMAGE_BRUSH_SVG("Starship/Common/Realtime", Icon16x16));
		Set("StaticMeshEditor.ReimportMesh", new IMAGE_BRUSH_SVG("Starship/StaticMeshEditor/ReimportMesh", Icon20x20));
		Set("StaticMeshEditor.ReimportMeshWithDialog", new IMAGE_BRUSH_SVG("Starship/StaticMeshEditor/ReimportMesh", Icon20x20));
		Set("StaticMeshEditor.SetShowBounds", new IMAGE_BRUSH_SVG("Starship/Common/SetShowBounds", Icon16x16));
		Set("StaticMeshEditor.SetDrawUVs", new IMAGE_BRUSH_SVG("Starship/Common/SetDrawUVs", Icon20x20));
		Set("StaticMeshEditor.SetShowCollision", new IMAGE_BRUSH_SVG("Starship/Common/Collision", Icon20x20));
		Set("StaticMeshEditor.SetShowGrid", new IMAGE_BRUSH_SVG("Starship/Common/Grid", Icon16x16));
		Set("StaticMeshEditor.ResetCamera", new IMAGE_BRUSH_SVG("Starship/Common/ResetCamera", Icon16x16));
		Set("StaticMeshEditor.SetShowPivot", new IMAGE_BRUSH_SVG("Starship/Common/SetShowPivot", Icon16x16));
		Set("StaticMeshEditor.SetShowSockets", new IMAGE_BRUSH_SVG( "Starship/Common/SetShowSockets", Icon16x16));
		Set("StaticMeshEditor.SaveThumbnail", new IMAGE_BRUSH_SVG( "Starship/Common/SaveThumbnail", Icon16x16));
		Set("StaticMeshEditor.SetShowNormals", new IMAGE_BRUSH_SVG( "Starship/Common/SetShowNormals", Icon16x16));
		Set("StaticMeshEditor.SetShowTangents", new IMAGE_BRUSH_SVG("Starship/Common/SetShowTangents", Icon16x16));
		Set("StaticMeshEditor.SetShowBinormals", new IMAGE_BRUSH_SVG("Starship/Common/SetShowBinormals", Icon16x16));
		Set("StaticMeshEditor.SetDrawAdditionalData", new IMAGE_BRUSH_SVG( "Starship/StaticMeshEditor/AdditionalData", Icon16x16));
		Set("StaticMeshEditor.SetShowVertices", new IMAGE_BRUSH_SVG("Starship/Common/SetShowVertices", Icon16x16));
		Set("StaticMeshEditor.ToggleShowPivots", new IMAGE_BRUSH_SVG("Starship/Common/SetShowPivot", Icon16x16));
		Set("StaticMeshEditor.ToggleShowSockets", new IMAGE_BRUSH_SVG("Starship/Common/SetShowSockets", Icon16x16));
		Set("StaticMeshEditor.ToggleShowNormals", new IMAGE_BRUSH_SVG("Starship/Common/SetShowNormals", Icon16x16));
		Set("StaticMeshEditor.ToggleShowTangents", new IMAGE_BRUSH_SVG("Starship/Common/SetShowTangents", Icon16x16));
		Set("StaticMeshEditor.ToggleShowBinormals", new IMAGE_BRUSH_SVG("Starship/Common/SetShowBinormals", Icon16x16));
		Set("StaticMeshEditor.ToggleShowBounds", new IMAGE_BRUSH_SVG("Starship/Common/SetShowBounds", Icon16x16));
		Set("StaticMeshEditor.ToggleShowGrids", new IMAGE_BRUSH_SVG("Starship/Common/Grid", Icon16x16));
		Set("StaticMeshEditor.ToggleShowVertices", new IMAGE_BRUSH_SVG("Starship/Common/SetShowVertices", Icon16x16));
		Set("StaticMeshEditor.ToggleShowWireframes", new IMAGE_BRUSH_SVG("Starship/Common/BrushWireframe", Icon16x16));
		Set("StaticMeshEditor.ToggleShowVertexColors", new IMAGE_BRUSH_SVG("Starship/Common/SetShowVertexColors", Icon16x16));

	}

	// Skeletal Mesh Editor
	{
		Set( "SkeletalMeshEditor.GroupSection", new BOX_BRUSH( "Common/RoundedSelection_16x", FMargin( 4.0f / 16.0f ) ) );
	}

	// Texture Editor
	{
		Set("TextureEditor.Tabs.Properties", new IMAGE_BRUSH("/Icons/icon_tab_SelectionDetails_16x", Icon16x16));
		
		Set("TextureEditor.RedChannel", new IMAGE_BRUSH( "Icons/icon_TextureEd_RedChannel_40x", Icon40x40));
		Set("TextureEditor.RedChannel.Small", new IMAGE_BRUSH( "Icons/icon_TextureEd_RedChannel_40x", Icon20x20));
		Set("TextureEditor.GreenChannel", new IMAGE_BRUSH( "Icons/icon_TextureEd_GreenChannel_40x", Icon40x40));
		Set("TextureEditor.GreenChannel.Small", new IMAGE_BRUSH( "Icons/icon_TextureEd_GreenChannel_40x", Icon20x20));
		Set("TextureEditor.BlueChannel", new IMAGE_BRUSH( "Icons/icon_TextureEd_BlueChannel_40x", Icon40x40));
		Set("TextureEditor.BlueChannel.Small", new IMAGE_BRUSH( "Icons/icon_TextureEd_BlueChannel_40x", Icon20x20));
		Set("TextureEditor.AlphaChannel", new IMAGE_BRUSH( "Icons/icon_TextureEd_AlphaChannel_40x", Icon40x40));
		Set("TextureEditor.AlphaChannel.Small", new IMAGE_BRUSH( "Icons/icon_TextureEd_AlphaChannel_40x", Icon20x20));
		Set("TextureEditor.Saturation", new IMAGE_BRUSH( "Icons/icon_TextureEd_Saturation_40x", Icon40x40));
		Set("TextureEditor.Saturation.Small", new IMAGE_BRUSH( "Icons/icon_TextureEd_Saturation_40x", Icon20x20));

		Set("TextureEditor.CompressNow", new IMAGE_BRUSH_SVG( "Starship/Common/Compress", Icon20x20));
		Set("TextureEditor.Reimport", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reimport", Icon20x20));

		FButtonStyle MipmapButtonStyle = 
			FButtonStyle(FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FButtonStyle>("Button"))
				.SetNormalPadding(FMargin(2, 2, 2, 2))
				.SetPressedPadding(FMargin(2, 3, 2, 1));

		Set("TextureEditor.MipmapButtonStyle", MipmapButtonStyle);

		const FLinearColor White80 = FLinearColor(1, 1, 1, .8f);

		const FCheckBoxStyle ChannelToggleButtonStyle = FCheckBoxStyle()
			.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
			.SetCheckedImage(FSlateRoundedBoxBrush(White80, CoreStyleConstants::InputFocusRadius))
			.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FLinearColor::White, CoreStyleConstants::InputFocusRadius))
			.SetCheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::White, CoreStyleConstants::InputFocusRadius))
			.SetUncheckedImage(FSlateRoundedBoxBrush(FStyleColors::Dropdown, CoreStyleConstants::InputFocusRadius))
			.SetUncheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, CoreStyleConstants::InputFocusRadius))
			.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::Hover, CoreStyleConstants::InputFocusRadius))
			.SetForegroundColor(FStyleColors::Foreground)
			.SetHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetPressedForegroundColor(FStyleColors::ForegroundHover)
			.SetCheckedForegroundColor(FStyleColors::Foreground)
			.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover)
			.SetPadding(FMargin(8.f, 4.f));

		FSlateFontInfo ChannelButtonFont = FStyleFonts::Get().NormalBold;
		ChannelButtonFont.Size = 12;
		Set("TextureEditor.ChannelButtonFont", ChannelButtonFont);

		Set("TextureEditor.ChannelButtonStyle", ChannelToggleButtonStyle);
	}

	// Cascade
	{
		Set( "Cascade.Tabs.Properties", new IMAGE_BRUSH( "/Icons/icon_tab_SelectionDetails_16x", Icon16x16 ) );
		
		Set( "Cascade.RestartSimulation", new IMAGE_BRUSH( "Icons/icon_Cascade_RestartSim_40x", Icon40x40 ) );
		Set( "Cascade.RestartInLevel", new IMAGE_BRUSH( "Icons/icon_Cascade_RestartInLevel_40x", Icon40x40 ) );
		Set( "Cascade.SaveThumbnailImage", new IMAGE_BRUSH( "Icons/icon_Cascade_Thumbnail_40x", Icon40x40 ) );
		Set( "Cascade.Undo", new IMAGE_BRUSH( "Icons/icon_Generic_Undo_40x", Icon40x40 ) );
		Set( "Cascade.Redo", new IMAGE_BRUSH( "Icons/icon_Generic_Redo_40x", Icon40x40 ) );
		Set( "Cascade.ToggleBounds", new IMAGE_BRUSH( "Icons/icon_Cascade_Bounds_40x", Icon40x40 ) );
		Set( "Cascade.ToggleOriginAxis", new IMAGE_BRUSH( "Icons/icon_Cascade_Axis_40x", Icon40x40 ) );
		Set( "Cascade.CascadeBackgroundColor", new IMAGE_BRUSH( "Icons/icon_Cascade_Color_40x", Icon40x40 ) );
		Set( "Cascade.RegenerateLowestLODDuplicatingHighest", new IMAGE_BRUSH( "Icons/icon_Cascade_RegenLOD1_40x", Icon40x40 ) );
		Set( "Cascade.RegenerateLowestLOD", new IMAGE_BRUSH( "Icons/icon_Cascade_RegenLOD2_40x", Icon40x40 ) );
		Set( "Cascade.JumpToHighestLOD", new IMAGE_BRUSH( "Icons/icon_Cascade_HighestLOD_40x", Icon40x40 ) );
		Set( "Cascade.JumpToHigherLOD", new IMAGE_BRUSH( "Icons/icon_Cascade_HigherLOD_40x", Icon40x40 ) );
		Set( "Cascade.AddLODAfterCurrent", new IMAGE_BRUSH( "Icons/icon_Cascade_AddLOD1_40x", Icon40x40 ) );
		Set( "Cascade.AddLODBeforeCurrent", new IMAGE_BRUSH( "Icons/icon_Cascade_AddLOD2_40x", Icon40x40 ) );
		Set( "Cascade.JumpToLowerLOD", new IMAGE_BRUSH( "Icons/icon_Cascade_LowerLOD_40x", Icon40x40 ) );
		Set( "Cascade.JumpToLowestLOD", new IMAGE_BRUSH( "Icons/icon_Cascade_LowestLOD_40x", Icon40x40 ) );
		Set( "Cascade.DeleteLOD", new IMAGE_BRUSH( "Icons/icon_Cascade_DeleteLOD_40x", Icon40x40 ) );
		
		Set( "Cascade.RestartSimulation.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_RestartSim_40x", Icon20x20 ) );
		Set( "Cascade.RestartInLevel.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_RestartInLevel_40x", Icon20x20 ) );
		Set( "Cascade.SaveThumbnailImage.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_Thumbnail_40x", Icon20x20 ) );
		Set( "Cascade.Undo.Small", new IMAGE_BRUSH( "Icons/icon_Generic_Undo_40x", Icon20x20 ) );
		Set( "Cascade.Redo.Small", new IMAGE_BRUSH( "Icons/icon_Generic_Redo_40x", Icon20x20 ) );
		Set( "Cascade.ToggleBounds.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_Bounds_40x", Icon20x20 ) );
		Set( "Cascade.ToggleOriginAxis.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_Axis_40x", Icon20x20 ) );
		Set( "Cascade.CascadeBackgroundColor.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_Color_40x", Icon20x20 ) );
		Set( "Cascade.RegenerateLowestLODDuplicatingHighest.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_RegenLOD1_40x", Icon20x20 ) );
		Set( "Cascade.RegenerateLowestLOD.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_RegenLOD2_40x", Icon20x20 ) );
		Set( "Cascade.JumpToHighestLOD.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_HighestLOD_40x", Icon20x20 ) );
		Set( "Cascade.JumpToHigherLOD.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_HigherLOD_40x", Icon20x20 ) );
		Set( "Cascade.AddLODAfterCurrent.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_AddLOD1_40x", Icon20x20 ) );
		Set( "Cascade.AddLODBeforeCurrent.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_AddLOD2_40x", Icon20x20 ) );
		Set( "Cascade.JumpToLowerLOD.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_LowerLOD_40x", Icon20x20 ) );
		Set( "Cascade.JumpToLowestLOD.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_LowestLOD_40x", Icon20x20 ) );
		Set( "Cascade.DeleteLOD.Small", new IMAGE_BRUSH( "Icons/icon_Cascade_DeleteLOD_40x", Icon20x20 ) );
	}

	// Level Script
	{
		Set( "LevelScript.Delete", new IMAGE_BRUSH( "Icons/icon_delete_16px", Icon16x16 ) );
	}

	// Curve Editor
	{
		Set("CurveAssetEditor.Tabs.Properties", new IMAGE_BRUSH("Icons/AssetIcons/CurveBase_16x", Icon16x16));
		
		Set("CurveEditor.FitHorizontally", new IMAGE_BRUSH("Icons/icon_CurveEditor_Horizontal_40x", Icon40x40));
		Set("CurveEditor.FitVertically", new IMAGE_BRUSH("Icons/icon_CurveEditor_Vertical_40x", Icon40x40));
		Set("CurveEditor.Fit", new IMAGE_BRUSH("Icons/icon_CurveEditor_ZoomToFit_40x", Icon40x40));
		Set("CurveEditor.PanMode", new IMAGE_BRUSH("Icons/icon_CurveEditor_Pan_40x", Icon40x40));
		Set("CurveEditor.ZoomMode", new IMAGE_BRUSH("Icons/icon_CurveEditor_Zoom_40x", Icon40x40));
		Set("CurveEditor.CurveAuto", new IMAGE_BRUSH("Icons/icon_CurveEditor_Auto_40x", Icon40x40));
		Set("CurveEditor.CurveAutoClamped", new IMAGE_BRUSH("Icons/icon_CurveEditor_AutoClamped_40x", Icon40x40));
		Set("CurveEditor.CurveUser", new IMAGE_BRUSH("Icons/icon_CurveEditor_User_40x", Icon40x40));
		Set("CurveEditor.CurveBreak", new IMAGE_BRUSH("Icons/icon_CurveEditor_Break_40x", Icon40x40));
		Set("CurveEditor.CurveWeight", new IMAGE_BRUSH("Icons/icon_CurveEditor_Break_40x", Icon40x40));
		Set("CurveEditor.CurveNonWeight", new IMAGE_BRUSH("Icons/icon_CurveEditor_Break_40x", Icon40x40));

		Set("CurveEditor.Linear", new IMAGE_BRUSH("Icons/icon_CurveEditor_Linear_40x", Icon40x40));
		Set("CurveEditor.Constant", new IMAGE_BRUSH("Icons/icon_CurveEditor_Constant_40x", Icon40x40));
		Set("CurveEditor.FlattenTangents", new IMAGE_BRUSH("Icons/icon_CurveEditor_Flatten_40x", Icon40x40));
		Set("CurveEditor.StraightenTangents", new IMAGE_BRUSH("Icons/icon_CurveEditor_Straighten_40x", Icon40x40));
		Set("CurveEditor.ShowAllTangents", new IMAGE_BRUSH("Icons/icon_CurveEditor_ShowAll_40x", Icon40x40));
		Set("CurveEditor.CreateTab", new IMAGE_BRUSH("Icons/icon_CurveEditor_Create_40x", Icon40x40));
		Set("CurveEditor.DeleteTab", new IMAGE_BRUSH("Icons/icon_CurveEditor_DeleteTab_40x", Icon40x40));

		Set("CurveEditor.FitHorizontally.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Horizontal_40x", Icon20x20));
		Set("CurveEditor.FitVertically.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Vertical_40x", Icon20x20));
		Set("CurveEditor.Fit.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_ZoomToFit_40x", Icon20x20));
		Set("CurveEditor.PanMode.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Pan_40x", Icon20x20));
		Set("CurveEditor.ZoomMode.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Zoom_40x", Icon20x20));
		Set("CurveEditor.CurveAuto.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Auto_40x", Icon20x20));
		Set("CurveEditor.CurveAutoClamped.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_AutoClamped_40x", Icon20x20));
		Set("CurveEditor.CurveUser.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_User_40x", Icon20x20));
		Set("CurveEditor.CurveBreak.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Break_40x", Icon20x20));
		Set("CurveEditor.CurveWeight.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Break_40x", Icon20x20));
		Set("CurveEditor.CurveNonWeight.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Break_40x", Icon20x20));

		Set("CurveEditor.Linear.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Linear_40x", Icon20x20));
		Set("CurveEditor.Constant.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Constant_40x", Icon20x20));
		Set("CurveEditor.FlattenTangents.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Flatten_40x", Icon20x20));
		Set("CurveEditor.StraightenTangents.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Straighten_40x", Icon20x20));
		Set("CurveEditor.ShowAllTangents.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_ShowAll_40x", Icon20x20));
		Set("CurveEditor.CreateTab.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_Create_40x", Icon20x20));
		Set("CurveEditor.DeleteTab.Small", new IMAGE_BRUSH("Icons/icon_CurveEditor_DeleteTab_40x", Icon20x20));

		Set("CurveEditor.Gradient.HandleDown", new BOX_BRUSH("Sequencer/ScrubHandleDown", FMargin(6.f / 13.f, 5 / 12.f, 6 / 13.f, 8 / 12.f)));
		Set("CurveEditor.Gradient.HandleUp", new BOX_BRUSH("Sequencer/ScrubHandleUp", FMargin(6.f / 13.f, 8 / 12.f, 6 / 13.f, 5 / 12.f)));
	}

	// New Curve Editor 
	{
		// Foreground colour that sliders, etc. should use. Generally, color when normal and hovered color should be the same (brighter as if hovered).
		const FSlateColor TweenColor(EStyleColor::ForegroundHover);
		Set("CurveEditor.TweenForeground", TweenColor);
		
		FToolBarStyle CurveEditorToolbar = FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FToolBarStyle>("AssetEditorToolbar");

		CurveEditorToolbar.SetButtonPadding(       FMargin(0.0f, 0.0f));
		CurveEditorToolbar.SetCheckBoxPadding(     FMargin(0.0f, 0.0f));
		CurveEditorToolbar.SetComboButtonPadding(  FMargin(0.0f, 0.0f));
		CurveEditorToolbar.SetIndentedBlockPadding(FMargin(0.0f, 0.0f));
		CurveEditorToolbar.SetBlockPadding(        FMargin(0.0f, 0.0f));
		CurveEditorToolbar.SetSeparatorPadding(    FMargin(2.0f, 0.0f));

		Set("CurveEditorToolBar", CurveEditorToolbar);

		// Some min sizes for toolbar combo buttons. We don't want the combo buttons to resize depending on the size of their contained label.
		CurveEditorToolbar.SetComboContentHorizontalAlignment(HAlign_Center);
		{
			constexpr float LongestTweenName = 90.f;
			
			CurveEditorToolbar.SetComboLabelMinWidth(68.f);
			Set("CurveEditorToolBar.ToolsCombo", CurveEditorToolbar);
			
			CurveEditorToolbar.SetComboLabelMinWidth(90.f);
			Set("CurveEditorToolBar.TangentMode", CurveEditorToolbar);

			FToolBarStyle TweenToolbar = CurveEditorToolbar;
			// Toolbar sliders should appear just a bright as when they are hovered.
			TweenToolbar.ButtonStyle.SetNormalForeground(TweenColor);
			TweenToolbar.ComboButtonStyle.ButtonStyle.SetNormalForeground(TweenColor);
			CurveEditorToolbar.SetComboLabelMinWidth(LongestTweenName);
			Set("CurveEditorTweenToolbar", CurveEditorToolbar);
			
			FLinearColor ViewportOverlayColor = FStyleColors::Input.GetSpecifiedColor();
			ViewportOverlayColor.A = 0.75f;
			Set("ControlRig.OverlayBrush", new FSlateRoundedBoxBrush(ViewportOverlayColor, 8.0f, FStyleColors::Dropdown, 0.0f));
			CurveEditorToolbar.SetComboLabelMinWidth(LongestTweenName);
			CurveEditorToolbar.SetAllowWrapButton(false);
			CurveEditorToolbar.SetBackground(*GetBrush("ControlRig.OverlayBrush"));
			Set("ControlRigTweenToolbar", CurveEditorToolbar);
		}

		Set("CurveEditor.KeyDetailWidth", 130.f);

		// Toolbar items that are allowed to be clipped. 0.f -> false, >=1.f -> true.
		Set("CurveEditor.AllowClipping.Sequencer.Save",						0.0f);
		Set("CurveEditor.AllowClipping.View",								0.0f);
		Set("CurveEditor.AllowClipping.Tools",								0.0f);
		Set("CurveEditor.AllowClipping.KeyDetails",							1.0f);
		Set("CurveEditor.AllowClipping.AxisSnapping",						1.0f);
		Set("CurveEditor.AllowClipping.ToggleInputSnapping",				1.0f);
		Set("CurveEditor.AllowClipping.TimeSnapping",						1.0f);
		Set("CurveEditor.AllowClipping.ToggleOutputSnapping",				1.0f);
		Set("CurveEditor.AllowClipping.GridSnapping",						1.0f);
		Set("CurveEditor.AllowClipping.TangentMenu",						1.0f);
		Set("CurveEditor.AllowClipping.InterpolationToggleWeighted",		1.0f);
		Set("CurveEditor.AllowClipping.FlattenTangents",					1.0f);
		Set("CurveEditor.AllowClipping.StraightenTangents",					1.0f);
		Set("CurveEditor.AllowClipping.CurvesMenu",							1.0f);
		Set("CurveEditor.AllowClipping.PromotedFilters",					1.0f);
		Set("CurveEditor.AllowClipping.OpenUserImplementableFilterWindow",	1.0f);
		Set("CurveEditor.AllowClipping.Tween.FunctionSelect",				1.0f);
		Set("CurveEditor.AllowClipping.Tween.Slider",						1.0f);
		Set("CurveEditor.AllowClipping.Tween.Overshoot",					1.0f);

		// Clipping priorities of toolbar items. High number = higher priority -> clipped last.
		Set("CurveEditor.ClipPriority.Sequencer.Save",						100.0f);
		Set("CurveEditor.ClipPriority.View",								100.0f);
		Set("CurveEditor.ClipPriority.Tools",								100.0f);
		Set("CurveEditor.ClipPriority.KeyDetails",							50.0f);
		Set("CurveEditor.ClipPriority.AxisSnapping",						40.0f);
		Set("CurveEditor.ClipPriority.ToggleInputSnapping",					41.0f);
		Set("CurveEditor.ClipPriority.TimeSnapping",						41.0f);
		Set("CurveEditor.ClipPriority.ToggleOutputSnapping",				42.0f);
		Set("CurveEditor.ClipPriority.GridSnapping",						42.0f);
		Set("CurveEditor.ClipPriority.TangentMenu",							30.0f);
		Set("CurveEditor.ClipPriority.InterpolationToggleWeighted",			21.0f);
		Set("CurveEditor.ClipPriority.FlattenTangents",						22.0f);
		Set("CurveEditor.ClipPriority.StraightenTangents",					23.0f);
		Set("CurveEditor.ClipPriority.CurvesMenu",							10.0f);
		Set("CurveEditor.ClipPriority.PromotedFilters",						11.0f);
		Set("CurveEditor.ClipPriority.OpenUserImplementableFilterWindow",	12.0f);
		Set("CurveEditor.ClipPriority.Tween.FunctionSelect",				71.0f);
		Set("CurveEditor.ClipPriority.Tween.Slider",						72.0f);
		Set("CurveEditor.ClipPriority.Tween.Overshoot",						70.0f);

		// Tab
		Set("GenericCurveEditor", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/CurveEditor", Icon20x20));
		Set("GenericCurveEditor.TabIcon", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/CurveEditorTab", Icon16x16, FLinearColor(1.f, 1.f, 1.f, 0.8f)));

		// Zoom / Framing
		Set("GenericCurveEditor.ZoomToFit", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/FramingSelected", Icon20x20));

		// Time/Value Snapping
		Set("GenericCurveEditor.ToggleInputSnapping", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/AdjustKeySnapFrameHorizontal", Icon20x20));
		Set("GenericCurveEditor.ToggleOutputSnapping", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/AdjustKeySnapFrameVertical", Icon20x20));

		// Flip Curve
		Set("GenericCurveEditor.FlipCurveHorizontal", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/FlipHorizontal", Icon20x20));
		Set("GenericCurveEditor.FlipCurveVertical", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/FlipVertical", Icon20x20));

		// Tangent Types
		Set("GenericCurveEditor.InterpolationCubicSmartAuto", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/TangentsCubicSmartAuto", Icon20x20));
		Set("GenericCurveEditor.InterpolationCubicAuto", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/TangentsCubicAuto", Icon20x20));
		Set("GenericCurveEditor.InterpolationCubicUser", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/TangentsCubicUser", Icon20x20));
		Set("GenericCurveEditor.InterpolationCubicBreak", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/TangentsCubicBreak", Icon20x20));
		Set("GenericCurveEditor.InterpolationToggleWeighted", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/TangentsWeighted", Icon20x20));
		Set("GenericCurveEditor.InterpolationLinear", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/TangentsLinear", Icon20x20));
		Set("GenericCurveEditor.InterpolationConstant", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/TangentsConstant", Icon20x20));
		Set("GenericCurveEditor.InterpolationMixed", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/CurveEditorTangentMultiSelected_20", Icon20x20));
		Set("GenericCurveEditor.InterpolationNoSelection", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/CurveEditorTangentUnselected_20", Icon20x20));

		// Tangent Modifications
		Set("GenericCurveEditor.FlattenTangents", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/MiscFlatten", Icon20x20));
		Set("GenericCurveEditor.StraightenTangents", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/MiscStraighten", Icon20x20));

		// View Modes
		Set("GenericCurveEditor.SetViewModeAbsolute", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/GraphViewAbsolute_20", Icon20x20));
		Set("GenericCurveEditor.SetViewModeStacked", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/GraphViewStack_20", Icon20x20));
		Set("GenericCurveEditor.SetViewModeNormalized", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/GraphViewNormalized_20", Icon20x20));

		// Axis Snapping
		Set("GenericCurveEditor.SetAxisSnappingNone", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/AdjustKeyMoveFree_20", Icon20x20));
		Set("GenericCurveEditor.SetAxisSnappingHorizontal", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/AdjustKeyLockValue_20", Icon20x20));
		Set("GenericCurveEditor.SetAxisSnappingVertical", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/AdjustKeyLockFrame_20", Icon20x20));

		// Deactivate Tool
		Set("GenericCurveEditor.DeactivateCurrentTool", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/Select", Icon20x20));

		// Filters
		Set("GenericCurveEditor.OpenUserImplementableFilterWindow", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/MiscFilters", Icon20x20));

		// Key Types
		Set("GenericCurveEditor.Key", new IMAGE_BRUSH("GenericCurveEditor/Keys/Diamond_Filled", FVector2f(11.0f, 11.0f))); 
		Set("GenericCurveEditor.ConstantKey", new IMAGE_BRUSH("GenericCurveEditor/Keys/Square_Filled", FVector2f(11.0f, 11.0f))); 
		Set("GenericCurveEditor.LinearKey", new IMAGE_BRUSH("GenericCurveEditor/Keys/Triangle_Filled", FVector2f(11.0f, 11.0f)));
		Set("GenericCurveEditor.CubicKey", new IMAGE_BRUSH("GenericCurveEditor/Keys/Diamond_Filled", FVector2f(11.0f, 11.0f)));
		Set("GenericCurveEditor.TangentHandle", new IMAGE_BRUSH("GenericCurveEditor/Keys/TangentHandle", Icon8x8));
		Set("GenericCurveEditor.WeightedTangentCubicKey", new IMAGE_BRUSH("GenericCurveEditor/Keys/Trapezoid_Filled", FVector2f(11.0f, 11.0f)));

		// Pre-Infinity
		Set("GenericCurveEditor.SetPreInfinityExtrapConstant", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PreInfinityConstant_20", Icon20x20));
		Set("GenericCurveEditor.SetPreInfinityExtrapCycle", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PreInfinityCycle_20", Icon20x20));
		Set("GenericCurveEditor.SetPreInfinityExtrapCycleWithOffset", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PreInfinityOffset_20", Icon20x20));
		Set("GenericCurveEditor.SetPreInfinityExtrapLinear", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PreInfinityLinear_20", Icon20x20));
		Set("GenericCurveEditor.SetPreInfinityExtrapOscillate", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PreInfinityOscillate_20", Icon20x20));
		Set("GenericCurveEditor.PreInfinityMixed", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PreInfinityMultipleSelected_20", Icon20x20));

		// Post-Infinity
		Set("GenericCurveEditor.SetPostInfinityExtrapConstant", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PostInfinityConstant_20", Icon20x20));
		Set("GenericCurveEditor.SetPostInfinityExtrapCycle", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PostInfinityCycle_20", Icon20x20));
		Set("GenericCurveEditor.SetPostInfinityExtrapCycleWithOffset", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PostInfinityOffset_20", Icon20x20));
		Set("GenericCurveEditor.SetPostInfinityExtrapLinear", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PostInfinityLinear_20", Icon20x20));
		Set("GenericCurveEditor.SetPostInfinityExtrapOscillate", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PostInfinityOscillate_20", Icon20x20));
		Set("GenericCurveEditor.PostInfinityMixed", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/PostInfinityMultipleSelected_20", Icon20x20));

		Set("GenericCurveEditor.Pin_Active", new IMAGE_BRUSH("Common/PushPin_Down", Icon16x16));
		Set("GenericCurveEditor.Pin_Inactive", new IMAGE_BRUSH("Common/PushPin_Up", Icon16x16));
		
		Set("GenericCurveEditor.Select", new IMAGE_BRUSH("GenericCurveEditor/Icons/SelectButton", Icon16x16));
		
		Set("GenericCurveEditor.Curves", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/CurveSettings_20", Icon20x20)); 

		Set("GenericCurveEditor.SmartSnapKeys", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/SmartSnapKeys_16", Icon16x16));
	}

	// Generic Curve Editor Tools
	{
		Set("CurveEditorTools.SetFocusPlaybackTime", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/FramingPlayback", Icon20x20));
		Set("CurveEditorTools.SetFocusPlaybackRange", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/FramingTimeRange", Icon20x20));

		Set("CurveEditorTools.ActivateTransformTool", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/ToolsTransform", Icon20x20));
		Set("CurveEditorTools.ActivateRetimeTool", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/ToolsRetime", Icon20x20));
		Set("CurveEditorTools.ActivateMultiScaleTool", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/ToolsMultiScale", Icon20x20));
		Set("CurveEditorTools.ActivateLatticeTool", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/ToolsLattice", Icon20x20));
	}

	// General Curve Icons
	{
		Set("Curve.ZoomToFit", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/FramingAll", Icon20x20));
		Set("Curve.ZoomToFitHorizontal", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/FramingHorizontal", Icon20x20));
		Set("Curve.ZoomToFitVertical", new IMAGE_BRUSH_SVG("Starship/GenericCurveEditor/FramingVertical", Icon20x20));
	}

	// PhysicsAssetEditor
	{
		Set( "PhysicsAssetEditor.Tabs.Properties", new IMAGE_BRUSH( "/Icons/icon_tab_SelectionDetails_16x", Icon16x16 ) );
		Set( "PhysicsAssetEditor.Tabs.Hierarchy", new IMAGE_BRUSH( "/Icons/levels_16x", Icon16x16 ) );
		Set( "PhysicsAssetEditor.Tabs.Profiles", new IMAGE_BRUSH_SVG( "Starship/AssetEditors/ProfileFolder", Icon16x16 ) );
		Set( "PhysicsAssetEditor.Tabs.Graph", new IMAGE_BRUSH( "/PhysicsAssetEditor/icon_GraphTab_16x", Icon16x16 ) );
		Set( "PhysicsAssetEditor.Tabs.Tools", new IMAGE_BRUSH( "/PhysicsAssetEditor/icon_ToolsTab_16x", Icon16x16 ) );

		Set( "PhysicsAssetEditor.EditingMode_Body", new IMAGE_BRUSH( "/PhysicsAssetEditor/icon_PHatMode_Body_40x", Icon40x40) );
		Set( "PhysicsAssetEditor.EditingMode_Constraint", new IMAGE_BRUSH( "/PhysicsAssetEditor/icon_PHatMode_Joint_40x", Icon40x40) );

		Set( "PhysicsAssetEditor.EditingMode_Body.Small", new IMAGE_BRUSH( "/PhysicsAssetEditor/icon_PHatMode_Body_40x", Icon20x20) );
		Set( "PhysicsAssetEditor.EditingMode_Constraint.Small", new IMAGE_BRUSH( "/PhysicsAssetEditor/icon_PHatMode_Joint_40x", Icon20x20) );

		Set( "PhysicsAssetEditor.SimulationNoGravity", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_PlaySimNoGravity_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.SimulationFloorCollision", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_EnableCollision_40x", Icon40x40));
		Set( "PhysicsAssetEditor.SelectedSimulation", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_PlaySimSelected_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.SimulationAll", new IMAGE_BRUSH_SVG("Starship/MainToolbar/simulate", Icon20x20));
		Set( "PhysicsAssetEditor.Undo", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Undo", Icon20x20));
		Set( "PhysicsAssetEditor.Redo", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Redo", Icon20x20));
		Set( "PhysicsAssetEditor.ChangeDefaultMesh", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_Mesh_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.ApplyPhysicalMaterial", new IMAGE_BRUSH_SVG("Starship/Persona/AnimationPhysicalMaterial", Icon20x20));
		Set( "PhysicsAssetEditor.CopyJointSettings", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_CopyJoints_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.PlayAnimation", new IMAGE_BRUSH_SVG( "Starship/Common/play", Icon20x20 ) );
		Set( "PhysicsAssetEditor.PhATTranslationMode", new IMAGE_BRUSH( "Icons/icon_translate_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.PhATRotationMode", new IMAGE_BRUSH( "Icons/icon_rotate_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.PhATScaleMode", new IMAGE_BRUSH( "Icons/icon_scale_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.Snap", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_Snap_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.CopyProperties", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_CopyProperties_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.DisableCollision", new IMAGE_BRUSH_SVG( "Starship/Common/DisableCollision", Icon20x20 ) );
		Set( "PhysicsAssetEditor.EnableCollision", new IMAGE_BRUSH_SVG("Starship/Common/EnableCollision", Icon20x20));
		Set( "PhysicsAssetEditor.DisableCollisionAll", new IMAGE_BRUSH_SVG("Starship/Common/DisableCollision", Icon20x20));
		Set( "PhysicsAssetEditor.EnableCollisionAll", new IMAGE_BRUSH_SVG("Starship/Common/EnableCollision", Icon20x20));
		Set( "PhysicsAssetEditor.WeldToBody", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_Weld_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.CreateBodyWithSphere", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_Sphere_40x", Icon40x40));
		Set( "PhysicsAssetEditor.CreateBodyWithSphyl", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_Sphyl_40x", Icon40x40));
		Set( "PhysicsAssetEditor.CreateBodyWithBox", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_Box_40x", Icon40x40));
		Set( "PhysicsAssetEditor.CreateBodyWithSingleConvex", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_Convex_40x", Icon40x40));
		Set( "PhysicsAssetEditor.CreateBodyWithMultiConvex", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_MultiConvex_40x", Icon40x40));
		Set( "PhysicsAssetEditor.CreateBodyWithTaperedCapsule", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_TaperedCapsule_40x", Icon40x40));
		Set( "PhysicsAssetEditor.CreateBodyShouldCreateConstraints", new IMAGE_BRUSH("PhysicsAssetEditor/Constraint_16x", Icon40x40));
		Set( "PhysicsAssetEditor.AddNewBody", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_NewBody_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.AddSphere", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_Sphere_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.AddSphyl", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_Sphyl_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.AddBox", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_Box_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.AddTaperedCapsule", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_TaperedCapsule_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.DeletePrimitive", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_DeletePrimitive_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.DuplicatePrimitive", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_DupePrim_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.ResetConstraint", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_ResetConstraint_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.SnapConstraint", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_SnapConstraint_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.SnapConstraintChildPosition", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_SnapConstraint_40x", Icon40x40));
		Set( "PhysicsAssetEditor.SnapConstraintChildOrientation", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_SnapConstraint_40x", Icon40x40));
		Set( "PhysicsAssetEditor.SnapConstraintParentPosition", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_SnapConstraint_40x", Icon40x40));
		Set( "PhysicsAssetEditor.SnapConstraintParentOrientation", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_SnapConstraint_40x", Icon40x40));
		Set( "PhysicsAssetEditor.SnapAllConstraints", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_SnapAll_40x", Icon40x40 ) );
		Set( "PhysicsAssetEditor.ConvertToBallAndSocket", new IMAGE_BRUSH_SVG("Starship/Persona/AnimationToBallAndSocket", Icon20x20));
		Set( "PhysicsAssetEditor.ConvertToHinge", new IMAGE_BRUSH_SVG("Starship/Persona/AnimationToHinge", Icon20x20));
		Set( "PhysicsAssetEditor.ConvertToPrismatic", new IMAGE_BRUSH_SVG("Starship/Persona/AnimationToPrismatic", Icon20x20));
		Set( "PhysicsAssetEditor.ConvertToSkeletal", new IMAGE_BRUSH_SVG("Starship/Persona/AnimationToSkeletal", Icon20x20));
		Set( "PhysicsAssetEditor.DeleteConstraint", new IMAGE_BRUSH( "PhysicsAssetEditor/icon_PhAT_DeleteConstraint_40x", Icon40x40 ) );

		Set("PhysicsAssetEditor.AddBodyToPhysicalAnimationProfile", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_NewBody_40x", Icon20x20));
		Set("PhysicsAssetEditor.RemoveBodyFromPhysicalAnimationProfile", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_DeletePrimitive_40x", Icon20x20));
		Set("PhysicsAssetEditor.AddConstraintToCurrentConstraintProfile", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PHatMode_Joint_40x", Icon20x20));
		Set("PhysicsAssetEditor.RemoveConstraintFromCurrentConstraintProfile", new IMAGE_BRUSH("PhysicsAssetEditor/icon_PhAT_DeleteConstraint_40x", Icon20x20));

		Set("PhysicsAssetEditor.Tree.BodyMultipleDefault", new IMAGE_BRUSH("PhysicsAssetEditor/BodyMultipleDefault_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.BodyMultipleKinematic", new IMAGE_BRUSH("PhysicsAssetEditor/BodyMultipleKinematic_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.BodyMultipleSimulated", new IMAGE_BRUSH("PhysicsAssetEditor/BodyMultipleSimulated_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.BodySingleDefault", new IMAGE_BRUSH("PhysicsAssetEditor/BodySingleDefault_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.BodySingleKinematic", new IMAGE_BRUSH("PhysicsAssetEditor/BodySingleKinematic_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.BodySingleSimulated", new IMAGE_BRUSH("PhysicsAssetEditor/BodySingleSimulated_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.Body", new IMAGE_BRUSH("PhysicsAssetEditor/BodyMultipleDefault_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.KinematicBody", new IMAGE_BRUSH("PhysicsAssetEditor/BodyMultipleKinematic_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.EmptyBody", new IMAGE_BRUSH("PhysicsAssetEditor/EmptyBody_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.Bone", new IMAGE_BRUSH("PhysicsAssetEditor/Bone_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.Sphere", new IMAGE_BRUSH("PhysicsAssetEditor/Sphere_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.Sphyl", new IMAGE_BRUSH("PhysicsAssetEditor/Sphyl_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.Box", new IMAGE_BRUSH("PhysicsAssetEditor/Box_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.Convex", new IMAGE_BRUSH("PhysicsAssetEditor/Convex_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.TaperedCapsule", new IMAGE_BRUSH("PhysicsAssetEditor/TaperedCapsule_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.Constraint", new IMAGE_BRUSH("PhysicsAssetEditor/BoneConstraint_16x", Icon16x16));
		Set("PhysicsAssetEditor.Tree.CrossConstraint", new IMAGE_BRUSH("PhysicsAssetEditor/BoneCrossConstraint_16x", Icon16x16));

		Set("PhysicsAssetEditor.BoneAssign", new IMAGE_BRUSH_SVG("Starship/Persona/BoneAssign", Icon20x20));
		Set("PhysicsAssetEditor.BoneUnassign", new IMAGE_BRUSH_SVG("Starship/Persona/BoneUnassign", Icon20x20));
		Set("PhysicsAssetEditor.BoneLocate", new IMAGE_BRUSH_SVG("Starship/Persona/BoneLocate", Icon20x20));

		Set("PhysicsAssetEditor.Tree.Font", DEFAULT_FONT("Regular", 10));

		Set("PhysicsAssetEditor.Graph.TextStyle", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(FLinearColor(0.0f, 0.0f, 0.0f, 1.0f))
			.SetFont(DEFAULT_FONT("Regular", 8)));

		Set("PhysicsAssetEditor.Graph.NodeBody", new BOX_BRUSH("PhysicsAssetEditor/NodeBody", FMargin(4.f / 64.f, 4.f / 64.f, 4.f / 64.f, 4.f / 64.f)));
		Set("PhysicsAssetEditor.Graph.NodeIcon", new IMAGE_BRUSH("PhysicsAssetEditor/Bone_16x", Icon16x16));
		Set("PhysicsAssetEditor.Graph.Pin.Background", new IMAGE_BRUSH("PhysicsAssetEditor/NodePin", Icon10x10));
		Set("PhysicsAssetEditor.Graph.Pin.BackgroundHovered", new IMAGE_BRUSH("PhysicsAssetEditor/NodePinHoverCue", Icon10x10));
		Set("PhysicsAssetEditor.Graph.Node.ShadowSelected", new BOX_BRUSH( "PhysicsAssetEditor/PhysicsNode_shadow_selected", FMargin(18.0f/64.0f) ) );
		Set("PhysicsAssetEditor.Graph.Node.Shadow", new BOX_BRUSH( "Graph/RegularNode_shadow", FMargin(18.0f/64.0f) ) );

		FEditableTextBoxStyle EditableTextBlock = NormalEditableTextBoxStyle
			.SetFont(NormalText.Font)
			.SetBackgroundImageNormal(FSlateNoResource())
			.SetBackgroundImageHovered(FSlateNoResource())
			.SetBackgroundImageFocused(FSlateNoResource())
			.SetBackgroundImageReadOnly(FSlateNoResource())
			.SetForegroundColor(FSlateColor::UseStyle());

		Set("PhysicsAssetEditor.Profiles.EditableTextBoxStyle", EditableTextBlock);

		Set("PhysicsAssetEditor.Profiles.Font", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 11))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0, 0, 0, 0.9f)));

		Set("PhysicsAssetEditor.Tools.Font", FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Bold", 11))
			.SetColorAndOpacity(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetHighlightColor(FLinearColor(1.0f, 1.0f, 1.0f))
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor(0.f, 0.f, 0.f, 0.9f)));

		FLinearColor Red = FLinearColor::Red;
		FLinearColor Red_Selected = FLinearColor::Red.Desaturate(0.75f);
		FLinearColor Red_Pressed = FLinearColor::Red.Desaturate(0.5f);

		const FCheckBoxStyle RedRadioButtonStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Red ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Red ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Red ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, Red ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, Red_Selected ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Red_Pressed ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Red ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Red_Selected ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Red_Pressed ) );

		Set( "PhysicsAssetEditor.RadioButtons.Red", RedRadioButtonStyle );

		FLinearColor Green = FLinearColor::Green;
		FLinearColor Green_Selected = FLinearColor::Green.Desaturate(0.75f);
		FLinearColor Green_Pressed = FLinearColor::Green.Desaturate(0.5f);

		const FCheckBoxStyle GreenRadioButtonStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Green ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Green ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Green ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, Green ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, Green_Selected ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Green_Pressed ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Green ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Green_Selected ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Green_Pressed ) );

		Set( "PhysicsAssetEditor.RadioButtons.Green", GreenRadioButtonStyle );

		FLinearColor Blue = FLinearColor::Blue;
		FLinearColor Blue_Selected = FLinearColor::Blue.Desaturate(0.75f);
		FLinearColor Blue_Pressed = FLinearColor::Blue.Desaturate(0.5f);

		const FCheckBoxStyle BlueRadioButtonStyle = FCheckBoxStyle()
			.SetUncheckedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Blue ) )
			.SetUncheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Blue ) )
			.SetUncheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Blue ) )
			.SetCheckedImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, Blue ) )
			.SetCheckedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Selected_16x", Icon16x16, Blue_Selected ) )
			.SetCheckedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Blue_Pressed ) )
			.SetUndeterminedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Blue ) )
			.SetUndeterminedHoveredImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Blue_Selected ) )
			.SetUndeterminedPressedImage( IMAGE_BRUSH( "Common/RadioButton_Unselected_16x", Icon16x16, Blue_Pressed ) );

		Set( "PhysicsAssetEditor.RadioButtons.Blue", BlueRadioButtonStyle );
	}
#endif // WITH_EDITOR
}

void FStarshipEditorStyle::FStyle::SetupUnsavedAssetsStyles()
{
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	Set("Assets.AllSaved", new CORE_IMAGE_BRUSH_SVG("Starship/Common/AllSavedAssets", Icon16x16, FStyleColors::Foreground));
	Set("Assets.Unsaved", new CORE_IMAGE_BRUSH_SVG("Starship/Common/UnsavedAssets", Icon16x16, FStyleColors::Foreground));
	Set("Assets.UnsavedWarning", new CORE_IMAGE_BRUSH_SVG("Starship/Common/UnsavedAssetsWarning", Icon16x16, FStyleColors::AccentYellow));
#endif
}

// These styles are oudated and exist for backwards compatibility, @see FRevisionControlStyleManager to use or the current revision control styles
void FStarshipEditorStyle::FStyle::SetupSourceControlStyles()
{
	// Most styles here have been replaced in FRevisionControlStyleManager, however some are still in the process of being transferred over and references updated etc.
	// If you want to use a revision control icon, use FRevisionControlStyleManager or add it there if it does not exist
	// If you want to add a new icon, add it in FRevisionControlStyleManager
	// If you want to modify an existing icon, look in FRevisionControlStyleManager instead and update it in both places for backwards compat (and if it doesn't exist there, add it there as a new icon)
	//Source Control
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	{
		Set("SourceControl.StatusIcon.On", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check-circle", Icon16x16, FStyleColors::AccentGreen));
		Set("SourceControl.StatusIcon.Error", new CORE_IMAGE_BRUSH_SVG("Starship/Common/alert-circle", Icon16x16, FStyleColors::AccentYellow));
		Set("SourceControl.StatusIcon.Off", new CORE_IMAGE_BRUSH_SVG("Starship/Common/reject", Icon16x16, FStyleColors::Foreground));
		Set("SourceControl.StatusIcon.Unknown", new CORE_IMAGE_BRUSH_SVG("Starship/Common/help", Icon16x16, FStyleColors::AccentYellow));

		Set("SourceControl.ChangelistsTab", new CORE_IMAGE_BRUSH_SVG("Starship/Common/check-circle", Icon16x16));
		Set("SourceControl.Changelist", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Changelist", Icon16x16, FStyleColors::AccentRed));
		Set("SourceControl.ShelvedChangelist", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Changelist", Icon16x16, FStyleColors::AccentBlue));
		Set("SourceControl.UncontrolledChangelist", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Changelist", Icon32x32, FStyleColors::AccentOrange));
		Set("SourceControl.UncontrolledChangelist_Small", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Changelist", Icon16x16, FStyleColors::AccentOrange));
		Set("SourceControl.OfflineFile_Small", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/Status/RevisionControl", Icon16x16, FStyleColors::AccentRed));

		Set("SourceControl.Add", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/SCC_ContentAdd",Icon16x16));
		Set("SourceControl.Edit", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/SCC_CheckedOut", Icon16x16));
		Set("SourceControl.Delete", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/SCC_MarkedForDelete", Icon16x16));
		Set("SourceControl.Branch", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/SCC_Branched", Icon16x16));
		Set("SourceControl.Integrate", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/SCC_Action_Integrate", Icon16x16));

		Set("SourceControl.LockOverlay", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Lock", Icon16x16));

		Set("SourceControl.Settings.StatusBorder", new BOX_BRUSH( "Common/GroupBorder", FMargin(4.0f/16.0f), FLinearColor(0.5f,0.5f,0.5f,1.0f)  ) );
		Set("SourceControl.Settings.StatusFont", FTextBlockStyle(NormalText).SetFont(DEFAULT_FONT( "Bold", 12 ) ));


		Set("SourceControl.ProgressWindow.Warning", new IMAGE_BRUSH( "Icons/alert", Icon32x32) );

		// Menu commands
		Set("SourceControl.Actions.Sync", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/arrow-right", Icon16x16));
		Set("SourceControl.Actions.Submit", new CORE_IMAGE_BRUSH_SVG( "Starship/Common/arrow-left", Icon16x16));
		Set("SourceControl.Actions.Diff", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/SCC_Action_Diff", Icon16x16));
		Set("SourceControl.Actions.Revert", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/icon_SCC_Revert", Icon16x16));
		Set("SourceControl.Actions.Connect", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/Status/RevisionControl", Icon16x16));
		Set("SourceControl.Actions.History", new CORE_IMAGE_BRUSH_SVG( "Starship/SourceControl/icon_SCC_History", Icon16x16));
		Set("SourceControl.Actions.Add", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16));
		Set("SourceControl.Actions.ChangeSettings", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/icon_SCC_Change_Source_Control_Settings", Icon16x16));
		Set("SourceControl.Actions.CheckOut", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16));
		Set("SourceControl.Actions.Refresh", new CORE_IMAGE_BRUSH_SVG("Starship/Common/Undo", Icon16x16));
		Set("SourceControl.Actions.NewChangelist", new CORE_IMAGE_BRUSH_SVG("Starship/Common/plus-circle", Icon16x16));

		// Diff colors
		Set("SourceControl.Diff.AdditionColor", FLinearColor(0.02f, 0.94f, 0.f));
		Set("SourceControl.Diff.SubtractionColor", FLinearColor(1.f, 0.16f, 0.16f));
		Set("SourceControl.Diff.MajorModificationColor", FLinearColor(0.04f, 0.87f, 1.f));
		Set("SourceControl.Diff.MinorModificationColor", FLinearColor(0.74f, 0.69f, 0.79f));
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
	// Perforce
	{
		Set("Perforce.CheckedOut", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16, FStyleColors::AccentRed));
		Set("Perforce.OpenForAdd", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ContentAdd", Icon16x16, FStyleColors::AccentRed));
		Set("Perforce.CheckedOutByOtherUser", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16, FStyleColors::AccentYellow));
		Set("Perforce.CheckedOutByOtherUserOtherBranch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16, FStyleColors::AccentBlue));
		Set("Perforce.ModifiedOtherBranch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ModifiedOtherBranch", Icon16x16, FStyleColors::AccentRed));
		Set("Perforce.MarkedForDelete", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_MarkedForDelete", Icon16x16, FStyleColors::AccentRed));
		Set("Perforce.NotAtHeadRevision", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ModifiedOtherBranch", Icon16x16, FStyleColors::AccentYellow));
		Set("Perforce.NotInDepot", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_NotInDepot", Icon16x16, FStyleColors::AccentYellow));
		Set("Perforce.Branched", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Branched", Icon16x16, FStyleColors::AccentGreen));
	}
	// Plastic SCM
	{
		Set("Plastic.CheckedOut", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16, FStyleColors::AccentRed));
		Set("Plastic.Changed", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16, FStyleColors::AccentWhite)); // custom
		Set("Plastic.OpenForAdd", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ContentAdd", Icon16x16, FStyleColors::AccentRed));
		Set("Plastic.CheckedOutByOtherUser", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16, FStyleColors::AccentYellow));
		Set("Plastic.ModifiedOtherBranch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ModifiedOtherBranch", Icon16x16, FStyleColors::AccentRed));
		Set("Plastic.MarkedForDelete", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_MarkedForDelete", Icon16x16, FStyleColors::AccentRed));
		Set("Plastic.LocallyDeleted", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_MarkedForDelete", Icon16x16, FStyleColors::AccentWhite)); // custom
		Set("Plastic.NotAtHeadRevision", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ModifiedOtherBranch", Icon16x16, FStyleColors::AccentYellow));
		Set("Plastic.Conflicted", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ModifiedOtherBranch", Icon16x16, FStyleColors::AccentRed)); // custom
		Set("Plastic.NotInDepot", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_NotInDepot", Icon16x16, FStyleColors::AccentYellow));
		Set("Plastic.Ignored", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_NotInDepot", Icon16x16, FStyleColors::AccentWhite)); // custom
		Set("Plastic.Branched", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Branched", Icon16x16, FStyleColors::AccentGreen));
		Set("Plastic.LocallyMoved", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Branched", Icon16x16, FStyleColors::AccentWhite)); // custom
	}
	// Subversion
	{
		Set("Subversion.CheckedOut", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16, FStyleColors::AccentRed));
		Set("Subversion.OpenForAdd", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ContentAdd", Icon16x16, FStyleColors::AccentRed));
		Set("Subversion.CheckedOutByOtherUser", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16, FStyleColors::AccentYellow));
		Set("Subversion.CheckedOutByOtherUserOtherBranch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_CheckedOut", Icon16x16, FStyleColors::AccentBlue));
		Set("Subversion.ModifiedOtherBranch", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ModifiedOtherBranch", Icon16x16, FStyleColors::AccentRed));
		Set("Subversion.MarkedForDelete", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_MarkedForDelete", Icon16x16, FStyleColors::AccentRed));
		Set("Subversion.NotAtHeadRevision", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_ModifiedOtherBranch", Icon16x16, FStyleColors::AccentYellow));
		Set("Subversion.NotInDepot", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_NotInDepot", Icon16x16, FStyleColors::AccentYellow));
		Set("Subversion.Branched", new CORE_IMAGE_BRUSH_SVG("Starship/SourceControl/SCC_Branched", Icon16x16, FStyleColors::AccentGreen));
	}
#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)
}

void FStarshipEditorStyle::FStyle::SetupAutomationStyles()
{
	//Automation
#if WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

	// Device Manager
	{
		Set( "DeviceDetails.Claim", new IMAGE_BRUSH_SVG( "Starship/DeviceManager/DeviceClaim", Icon20x20) );
		Set( "DeviceDetails.Release", new IMAGE_BRUSH_SVG("Starship/DeviceManager/DeviceRelease", Icon20x20));
		Set( "DeviceDetails.Remove", new IMAGE_BRUSH_SVG("Starship/DeviceManager/DeviceRemove", Icon20x20));
		Set( "DeviceDetails.Share", new IMAGE_BRUSH_SVG("Starship/DeviceManager/DeviceShare", Icon20x20));

		Set( "DeviceDetails.Connect", new IMAGE_BRUSH_SVG("Starship/DeviceManager/CircleCheck_20", Icon20x20));
		Set( "DeviceDetails.Disconnect", new IMAGE_BRUSH_SVG("Starship/DeviceManager/CircleX_20", Icon20x20));

		Set( "DeviceDetails.PowerOn", new IMAGE_BRUSH_SVG("Starship/DeviceManager/PowerOn_20", Icon20x20));
		Set( "DeviceDetails.PowerOff", new IMAGE_BRUSH_SVG("Starship/DeviceManager/CircleMinus_20", Icon20x20));
		Set( "DeviceDetails.PowerOffForce", new IMAGE_BRUSH_SVG("Starship/DeviceManager/CircleMinus_20", Icon20x20));
		Set( "DeviceDetails.Reboot", new IMAGE_BRUSH_SVG("Starship/DeviceManager/Reboot", Icon20x20));

		Set( "DeviceDetails.TabIcon", new IMAGE_BRUSH_SVG( "Starship/Common/DeviceManager", Icon16x16 ) );
		Set( "DeviceDetails.Tabs.Tools", new CORE_IMAGE_BRUSH( "/Icons/icon_tab_Tools_16x", Icon16x16 ) );
		Set( "DeviceDetails.Tabs.ProfileEditor", new IMAGE_BRUSH_SVG( "Starship/Common/DeviceProfiles", Icon16x16 ) );
		Set( "DeviceDetails.Tabs.ProfileEditorSingleProfile", new IMAGE_BRUSH( "/Icons/icon_tab_DeviceProfileEditor_16x", Icon16x16 ) );

		// Todo: Remove this button style once Property Editor has been reskinned
		const FButtonStyle DeviceProfileCellButton = FButtonStyle(FStarshipCoreStyle::GetCoreStyle().GetWidgetStyle<FButtonStyle>("NoBorder"))
			.SetNormalForeground(FStyleColors::AccentBlack)
			.SetHoveredForeground(FLinearColor::FromSRGBColor(FColor::FromHex("#868686FF")))
			.SetPressedForeground(FLinearColor::FromSRGBColor(FColor::FromHex("#868686FF")));
		
		Set( "DeviceDetails.EditButton", DeviceProfileCellButton);

		Set( "DeviceDetails.WIFI.IOS", new IMAGE_BRUSH("Starship/DeviceManager/AppleWifi_128x", Icon128x128));
		Set( "DeviceDetails.USB.IOS", new IMAGE_BRUSH("Starship/DeviceManager/AppleUsb_128x", Icon128x128));
		Set( "DeviceDetails.WIFI.TVOS", new IMAGE_BRUSH("Starship/DeviceManager/TVOS_wifi_128x", Icon128x128));
		Set( "DeviceDetails.USB.TVOS", new IMAGE_BRUSH("Starship/DeviceManager/TVOS_usb_128x", Icon128x128));
	}

	// Settings Editor
	{
		Set( "SettingsEditor.Collision_Engine", new IMAGE_BRUSH("Icons/icon_Cascade_RestartSim_40x", Icon16x16));
		Set( "SettingsEditor.Collision_Game", new IMAGE_BRUSH_SVG("Starship/Common/Realtime", Icon16x16));

		// Settings editor
		Set("SettingsEditor.GoodIcon", new IMAGE_BRUSH("Settings/Settings_Good", Icon40x40));
		Set("SettingsEditor.WarningIcon", new IMAGE_BRUSH("Settings/Settings_Warning", Icon40x40));

		Set("SettingsEditor.CheckoutWarningBorder", new BOX_BRUSH( "Common/GroupBorderLight", FMargin(4.0f/16.0f) ) );

		Set("SettingsEditor.CatgoryAndSectionFont", DEFAULT_FONT("Regular", 18));
		Set("SettingsEditor.TopLevelObjectFontStyle", DEFAULT_FONT("Bold", 12));
	}

	{
		// Navigation defaults
		const FLinearColor NavHyperlinkColor(0.03847f, 0.33446f, 1.0f);
		const FTextBlockStyle NavigationHyperlinkText = FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 12))
			.SetColorAndOpacity(NavHyperlinkColor);

		const FButtonStyle NavigationHyperlinkButton = FButtonStyle()
			.SetNormal(BORDER_BRUSH("Old/HyperlinkDotted", FMargin(0, 0, 0, 3 / 16.0f), NavHyperlinkColor))
			.SetPressed(FSlateNoResource())
			.SetHovered(BORDER_BRUSH("Old/HyperlinkUnderline", FMargin(0, 0, 0, 3 / 16.0f), NavHyperlinkColor));

		FHyperlinkStyle NavigationHyperlink = FHyperlinkStyle()
			.SetUnderlineStyle(NavigationHyperlinkButton)
			.SetTextStyle(NavigationHyperlinkText)
			.SetPadding(FMargin(0.0f));

		Set("NavigationHyperlink", NavigationHyperlink);
	}

#endif // WITH_EDITOR || (IS_PROGRAM && WITH_UNREAL_DEVELOPER_TOOLS)

	// External image picker
	{
		Set("ExternalImagePicker.BlankImage", new IMAGE_BRUSH( "Icons/BlankIcon", Icon16x16 ) );
		Set("ExternalImagePicker.ThumbnailShadow", new BOX_BRUSH( "ContentBrowser/ThumbnailShadow" , FMargin( 4.0f / 64.0f ) ) );
		Set("ExternalImagePicker.PickImageButton", new IMAGE_BRUSH( "Icons/ellipsis_12x", Icon12x12 ) );
		Set("ExternalImagePicker.GenerateImageButton", new IMAGE_BRUSH("Icons/wrench_16x", Icon12x12));
	}


	{

		Set("FBXIcon.StaticMesh", new IMAGE_BRUSH("Icons/FBX/StaticMesh_16x", Icon16x16));
		Set("FBXIcon.SkeletalMesh", new IMAGE_BRUSH("Icons/FBX/SkeletalMesh_16x", Icon16x16));
		Set("FBXIcon.Animation", new IMAGE_BRUSH( "Icons/FBX/Animation_16px", Icon16x16 ) );
		Set("FBXIcon.ImportOptionsOverride", new IMAGE_BRUSH("Icons/FBX/FbxImportOptionsOverride_7x16px", Icon7x16));
		Set("FBXIcon.ImportOptionsDefault", new IMAGE_BRUSH("Icons/FBX/FbxImportOptionsDefault_7x16px", Icon7x16));

		Set("FBXIcon.ReimportAdded", new IMAGE_BRUSH("Icons/FBX/FbxReimportAdded_16x16px", Icon16x16));
		Set("FBXIcon.ReimportRemoved", new IMAGE_BRUSH("Icons/FBX/FbxReimportRemoved_16x16px", Icon16x16));
		Set("FBXIcon.ReimportSame", new IMAGE_BRUSH("Icons/FBX/FbxReimportSame_16x16px", Icon16x16));
		Set("FBXIcon.ReimportAddedContent", new IMAGE_BRUSH("Icons/FBX/FbxReimportAddedContent_16x16px", Icon16x16));
		Set("FBXIcon.ReimportRemovedContent", new IMAGE_BRUSH("Icons/FBX/FbxReimportRemovedContent_16x16px", Icon16x16));
		Set("FBXIcon.ReimportSameContent", new IMAGE_BRUSH("Icons/FBX/FbxReimportSameContent_16x16px", Icon16x16));
		Set("FBXIcon.ReimportError", new IMAGE_BRUSH("Icons/FBX/FbxReimportError_16x16px", Icon16x16));

		Set("FBXIcon.ReimportCompareAdd", new IMAGE_BRUSH("Icons/FBX/FbxReimportCompare-Add_16x16px", Icon16x16));
		Set("FBXIcon.ReimportCompareRemoved", new IMAGE_BRUSH("Icons/FBX/FbxReimportCompare-Remove_16x16px", Icon16x16));

		const FTextBlockStyle FBXLargeFont =
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 12))
			.SetColorAndOpacity(FSlateColor::UseForeground())
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor::Black);

		Set("FBXLargeFont", FBXLargeFont);

		const FTextBlockStyle FBXMediumFont =
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 11))
			.SetColorAndOpacity(FSlateColor::UseForeground())
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor::Black);

		Set("FBXMediumFont", FBXMediumFont);

		const FTextBlockStyle FBXSmallFont =
			FTextBlockStyle(NormalText)
			.SetFont(DEFAULT_FONT("Regular", 10))
			.SetColorAndOpacity(FSlateColor::UseForeground())
			.SetShadowOffset(FVector2f::UnitVector)
			.SetShadowColorAndOpacity(FLinearColor::Black);

		Set("FBXSmallFont", FBXSmallFont);
	}

	// Asset Dialog
	{
		Set("AssetDialog.ErrorLabelBorder", new FSlateColorBrush(FStyleColors::AccentRed));
	}
}

void FStarshipEditorStyle::FStyle::SetupUMGEditorStyles()
{
	Set("WidgetDesigner.LayoutTransform", new IMAGE_BRUSH("Icons/UMG/Layout_TransformMode_16x", Icon16x16));
	Set("WidgetDesigner.LayoutTransform.Small", new IMAGE_BRUSH("Icons/UMG/Layout_TransformMode_16x", Icon16x16));
	Set("WidgetDesigner.RenderTransform", new IMAGE_BRUSH("Icons/UMG/Render_TransformMode_16x", Icon16x16));
	Set("WidgetDesigner.RenderTransform.Small", new IMAGE_BRUSH("Icons/UMG/Render_TransformMode_16x", Icon16x16));
	Set("WidgetDesigner.ToggleOutlines", new IMAGE_BRUSH("Icons/UMG/ToggleOutlines.Small", Icon16x16));
	Set("WidgetDesigner.ToggleRespectLocks", new CORE_IMAGE_BRUSH_SVG("Starship/Common/lock", Icon16x16));

	Set("WidgetDesigner.ToggleLocalizationPreview", new IMAGE_BRUSH("Icons/icon_localization_white_16x", Icon16x16));

	Set("WidgetDesigner.LocationGridSnap", new IMAGE_BRUSH_SVG("Starship/EditorViewport/grid", Icon16x16));
	Set("WidgetDesigner.RotationGridSnap", new IMAGE_BRUSH("Old/LevelEditor/RotationGridSnap", Icon16x16));

	Set("WidgetDesigner.WidgetVisible", new IMAGE_BRUSH("/Icons/icon_layer_visible", Icon16x16));
	Set("WidgetDesigner.WidgetHidden", new IMAGE_BRUSH("/Icons/icon_layer_not_visible", Icon16x16));

	Set("UMGEditor.ZoomToFit", new IMAGE_BRUSH("GenericCurveEditor/Icons/FramingSelected_48x", Icon16x16));

	Set("UMGEditor.ScreenOutline", new BOX_BRUSH(TEXT("Icons/UMG/ScreenOutline"), FMargin(0.25f) ));

	Set("UMGEditor.TransformHandle", new IMAGE_BRUSH("Icons/UMG/TransformHandle", Icon8x8));
	Set("UMGEditor.ResizeAreaHandle", new IMAGE_BRUSH("Icons/UMG/ResizeAreaHandle", Icon20x20));

	Set("UMGEditor.AnchorGizmo.Center", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/center", Icon16x16));
	Set("UMGEditor.AnchorGizmo.Center.Hovered", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/center", Icon16x16, FLinearColor::Green));
	
	Set("UMGEditor.AnchorGizmo.Left", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/left", FVector2f(32.f, 16.f)));
	Set("UMGEditor.AnchorGizmo.Left.Hovered", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/left", FVector2f(32.f, 16.f), FLinearColor::Green));
	Set("UMGEditor.AnchorGizmo.Right", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/right", FVector2f(32.f, 16.f)));
	Set("UMGEditor.AnchorGizmo.Right.Hovered", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/right", FVector2f(32.f, 16.f), FLinearColor::Green));
	
	Set("UMGEditor.AnchorGizmo.Top", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/top", FVector2f(16.f, 32.f)));
	Set("UMGEditor.AnchorGizmo.Top.Hovered", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/top", FVector2f(16.f, 32.f), FLinearColor::Green));
	Set("UMGEditor.AnchorGizmo.Bottom", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/bottom", FVector2f(16.f, 32.f)));
	Set("UMGEditor.AnchorGizmo.Bottom.Hovered", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/bottom", FVector2f(16.f, 32.f), FLinearColor::Green));

	Set("UMGEditor.AnchorGizmo.TopLeft", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/topleft", Icon24x24));
	Set("UMGEditor.AnchorGizmo.TopLeft.Hovered", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/topleft", Icon24x24, FLinearColor::Green));

	Set("UMGEditor.AnchorGizmo.TopRight", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/topright", Icon24x24));
	Set("UMGEditor.AnchorGizmo.TopRight.Hovered", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/topright", Icon24x24, FLinearColor::Green));

	Set("UMGEditor.AnchorGizmo.BottomLeft", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/bottomleft", Icon24x24));
	Set("UMGEditor.AnchorGizmo.BottomLeft.Hovered", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/bottomleft", Icon24x24, FLinearColor::Green));

	Set("UMGEditor.AnchorGizmo.BottomRight", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/bottomright", Icon24x24));
	Set("UMGEditor.AnchorGizmo.BottomRight.Hovered", new IMAGE_BRUSH("Icons/UMG/AnchorGizmo/bottomright", Icon24x24, FLinearColor::Green));


	Set("UMGEditor.AnchoredWidget", new BOX_BRUSH("Common/Button", Icon32x32, 8.0f / 32.0f));
	Set("UMGEditor.AnchoredWidgetAlignment", new IMAGE_BRUSH("Icons/icon_tab_DeviceManager_16x", Icon8x8));
	

	Set("UMGEditor.PaletteHeader", FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Header))
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::Header))
		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Header))
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::Header))
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(FSlateNoResource())
		.SetActiveHoveredBrush(FSlateNoResource())
		.SetInactiveBrush(FSlateNoResource())
		.SetInactiveHoveredBrush(FSlateNoResource())
		.SetTextColor(DefaultForeground)
		.SetSelectedTextColor(InvertedForeground)
	);

	Set("UMGEditor.LibraryView", FTableRowStyle()
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Background))
		.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::Background))
		.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Background))
		.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::Background))
		.SetSelectorFocusedBrush(FSlateNoResource())
		.SetActiveBrush(FSlateNoResource())
		.SetActiveHoveredBrush(FSlateNoResource())
		.SetInactiveBrush(FSlateNoResource())
		.SetInactiveHoveredBrush(FSlateNoResource())
		.SetTextColor(DefaultForeground)
		.SetSelectedTextColor(InvertedForeground)
	);

	// Style of the favorite toggle
	const FCheckBoxStyle UMGEditorFavoriteToggleStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::CheckBox)
		.SetUncheckedImage(IMAGE_BRUSH("Icons/EmptyStar_16x", Icon10x10, FLinearColor(0.8f, 0.8f, 0.8f, 1.f)))
		.SetUncheckedHoveredImage(IMAGE_BRUSH("Icons/EmptyStar_16x", Icon10x10, FLinearColor(2.5f, 2.5f, 2.5f, 1.f)))
		.SetUncheckedPressedImage(IMAGE_BRUSH("Icons/EmptyStar_16x", Icon10x10, FLinearColor(0.8f, 0.8f, 0.8f, 1.f)))
		.SetCheckedImage(IMAGE_BRUSH("Icons/Star_16x", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)))
		.SetCheckedHoveredImage(IMAGE_BRUSH("Icons/Star_16x", Icon10x10, FLinearColor(0.4f, 0.4f, 0.4f, 1.f)))
		.SetCheckedPressedImage(IMAGE_BRUSH("Icons/Star_16x", Icon10x10, FLinearColor(0.2f, 0.2f, 0.2f, 1.f)));
	Set("UMGEditor.Palette.FavoriteToggleStyle", UMGEditorFavoriteToggleStyle);

	Set("HorizontalAlignment_Left", new IMAGE_BRUSH("Icons/UMG/Alignment/Horizontal_Left", Icon16x16));
	Set("HorizontalAlignment_Center", new IMAGE_BRUSH("Icons/UMG/Alignment/Horizontal_Center", Icon16x16));
	Set("HorizontalAlignment_Right", new IMAGE_BRUSH("Icons/UMG/Alignment/Horizontal_Right", Icon16x16));
	Set("HorizontalAlignment_Fill", new IMAGE_BRUSH("Icons/UMG/Alignment/Horizontal_Fill", Icon16x16));

	Set("VerticalAlignment_Top", new IMAGE_BRUSH("Icons/UMG/Alignment/Vertical_Top", Icon16x16));
	Set("VerticalAlignment_Center", new IMAGE_BRUSH("Icons/UMG/Alignment/Vertical_Center", Icon16x16));
	Set("VerticalAlignment_Bottom", new IMAGE_BRUSH("Icons/UMG/Alignment/Vertical_Bottom", Icon16x16));
	Set("VerticalAlignment_Fill", new IMAGE_BRUSH("Icons/UMG/Alignment/Vertical_Fill", Icon16x16));

	const FTextBlockStyle NoAnimationFont =
		FTextBlockStyle(NormalText)
		.SetFont(DEFAULT_FONT("Regular", 18))
		.SetColorAndOpacity(FSlateColor::UseForeground())
		.SetShadowOffset(FVector2f::UnitVector)
		.SetShadowColorAndOpacity(FLinearColor::Black);


	Set("UMGEditor.AddAnimationIcon", new IMAGE_BRUSH("Icons/PlusSymbol_12x", Icon12x12, FLinearColor(.05f,.05f,.05f) ) );
	Set("UMGEditor.NoAnimationFont", NoAnimationFont);

	Set("UMGEditor.SwitchToDesigner", new IMAGE_BRUSH("UMG/Designer_40x", Icon20x20));

	Set("UMGEditor.AnchorGrid", new IMAGE_BRUSH("Icons/UMG/AnchorGrid", Icon10x10, FLinearColor(.1f, .1f, .1f, 0.5f), ESlateBrushTileType::Both ));

	Set("UMGEditor.DPISettings", new IMAGE_BRUSH("Icons/UMG/SettingsButton", Icon16x16));

	Set("UMGEditor.DesignerMessageBorder", new BOX_BRUSH("/UMG/MessageRoundedBorder", FMargin(18.0f / 64.0f)));

	Set("UMGEditor.OrientLandscape", new IMAGE_BRUSH("Icons/UMG/Icon_Landscape_v2", Icon16x16));
	Set("UMGEditor.OrientPortrait", new IMAGE_BRUSH("Icons/UMG/Icon_Portrait_v2", Icon16x16));
	Set("UMGEditor.Mirror", new IMAGE_BRUSH("Icons/UMG/Icon_Mirror_v3", Icon16x16));

	Set("UMGEditor.ResizeResolutionFont", DEFAULT_FONT("Bold", 10));
	Set("UMGEditor.CategoryIcon", new IMAGE_BRUSH("Icons/hiererchy_16x", Icon16x16));
	Set("UMGEditor.AnimTabIcon", new IMAGE_BRUSH_SVG("Starship/MainToolbar/cinematics", Icon16x16));
}

void FStarshipEditorStyle::FStyle::SetupHomeScreenStyles()
{
	Set("HomeScreen.UnrealLogo", new IMAGE_BRUSH_SVG("Starship/HomeScreen/UnrealLogo", FVector2D(99.f, 120.f)));
	Set("HomeScreen.Home", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Home", Icon20x20));
	Set("HomeScreen.News", new IMAGE_BRUSH_SVG("Starship/HomeScreen/News", Icon20x20));
	Set("HomeScreen.Rocket", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Rocket", Icon20x20));
	Set("HomeScreen.Archive", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Archive", Icon20x20));
	Set("HomeScreen.Forum", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Forum", Icon20x20));
	Set("HomeScreen.Documentation", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Documentation", Icon20x20));
	Set("HomeScreen.Tutorial", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Tutorial", Icon20x20));
	Set("HomeScreen.Roadmap", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Roadmap", Icon20x20));
	Set("HomeScreen.ReleaseNotes", new IMAGE_BRUSH_SVG("Starship/HomeScreen/ReleaseNotes", Icon20x20));
	Set("HomeScreen.CreatorRules", new IMAGE_BRUSH_SVG("Starship/HomeScreen/ParentChild", Icon20x20));
	Set("HomeScreen.Support", new IMAGE_BRUSH_SVG("Starship/HomeScreen/HeadphonesMic", Icon20x20));
	Set("HomeScreen.PlusCircle", new IMAGE_BRUSH_SVG("Starship/HomeScreen/PlusCircleEmpty", Icon16x16));
	Set("HomeScreen.FolderOpen", new IMAGE_BRUSH_SVG("Starship/HomeScreen/FolderOpen", Icon16x16));

	Set("SocialMedia.Facebook", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Facebook", Icon20x20));
	Set("SocialMedia.Twitter", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Twitter", Icon20x20));
	Set("SocialMedia.YouTube", new IMAGE_BRUSH_SVG("Starship/HomeScreen/YouTube", Icon20x20));
	Set("SocialMedia.Twitch", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Twitch", Icon20x20));
	Set("SocialMedia.Instagram", new IMAGE_BRUSH_SVG("Starship/HomeScreen/Instagram", Icon20x20));

	Set("HomeScreen.RecentProjects.Background", new FSlateRoundedBoxBrush(COLOR("#FFFFFF0D"), 16.f, FStyleColors::White25, 1.f));

	Set("HomeScreen.NoInternet.Border", new FSlateRoundedBoxBrush(COLOR("#FFFFFF0D"), 8.f, COLOR("#FFFFFF59"), 1.f));
	Set("HomeScreen.NoInternet.Icon", new IMAGE_BRUSH_SVG("Starship/HomeScreen/NoInternet", FVector2D(24.f, 24.f)));

	FSlateFontInfo NewAndCreateProjectButtonFont = DEFAULT_FONT("Regular", 11);
    Set("HomeScreen.Font.NewAndCreateButton", NewAndCreateProjectButtonFont);

	FSlateFontInfo DefaultFontSize = DEFAULT_FONT("Regular", 12);
	Set("HomeScreen.Font.DefaultSize", DefaultFontSize);

	FSlateFontInfo DefaultBoldFontSize = DEFAULT_FONT("Bold", 12);
	Set("HomeScreen.Font.DefaultBoldSize", DefaultBoldFontSize);

	FSlateFontInfo RecentProjectFont = DEFAULT_FONT("Bold", 18);
	Set("HomeScreen.Font.RecentProject", RecentProjectFont);

	FComboButtonStyle LoadOnStartupStyle = FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("ComboButton");
	LoadOnStartupStyle.ButtonStyle = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(COLOR("#FFFFFF0D"), 6.f, COLOR("#FFFFFF59"), 1.f))
		.SetHovered(FSlateRoundedBoxBrush(COLOR("#FFFFFF0D"), 6.f, COLOR("#FFFFFF59"), 1.f))
		.SetPressed(FSlateRoundedBoxBrush(COLOR("#FFFFFF0D"), 6.f, COLOR("#FFFFFF59"), 1.f))
		.SetHoveredForeground(FStyleColors::White)
		.SetNormalPadding(FMargin(12.0f, 4.0f))
		.SetPressedPadding(FMargin(12.0f, 4.0f));

	LoadOnStartupStyle.SetMenuBorderBrush(FSlateRoundedBoxBrush(FStyleColors::Panel, 0.f, FStyleColors::White25, 1.f));
	LoadOnStartupStyle.SetMenuBorderPadding(0.f);

	Set("HomeScreen.ComboButton.HoveredEntry", new FSlateRoundedBoxBrush(FStyleColors::Input, 0.f));
	Set("HomeScreen.ComboButton", LoadOnStartupStyle);

	FButtonStyle LoadOnStartupMenuButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("NoBorder");
	LoadOnStartupMenuButtonStyle.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f));
	Set("HomeScreen.ComboButton.MenuButton", LoadOnStartupMenuButtonStyle);

	FButtonStyle CreateNewProjectButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::AccentBlue, 4.f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::AccentBlue, 4.f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::AccentBlue, 4.f))
		.SetHoveredForeground(FStyleColors::Black)
		.SetNormalForeground(FStyleColors::Black)
		.SetPressedForeground(FStyleColors::Black)
		.SetNormalPadding(FMargin(0.f, 4.0f))
		.SetPressedPadding(FMargin(0.f, 4.0f));
	Set("HomeScreen.CreateNewProjectButton", CreateNewProjectButton);

	FButtonStyle MyFolderButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, 4.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, 4.0f))
		.SetHoveredForeground(FStyleColors::White)
		.SetNormalForeground(FStyleColors::White)
		.SetPressedForeground(FStyleColors::White)
		.SetNormalPadding(FMargin(0.f, 4.0f))
		.SetPressedPadding(FMargin(0.f, 4.0f));
	Set("HomeScreen.MyFolderButton", MyFolderButton);

	FButtonStyle SeeAllProjectsButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::Transparent, 8.0f, COLOR("565659FF"), 1.f))
		.SetHovered(FSlateRoundedBoxBrush(COLOR("404044FF"), 8.0f, COLOR("FFFFFFFF"), 1.f))
		.SetPressed(FSlateRoundedBoxBrush(COLOR("404044FF"), 8.0f, COLOR("FFFFFFFF"), 1.f))
		.SetHoveredForeground(FStyleColors::White)
		.SetNormalForeground(COLOR("EDEDEDFF"))
		.SetPressedForeground(FStyleColors::White)
		.SetNormalPadding(FMargin(0.f, 4.0f))
		.SetPressedPadding(FMargin(0.f, 4.0f));
	Set("HomeScreen.SeeAllProjectsButton", SeeAllProjectsButton);

	FButtonStyle ReconnectButton = FButtonStyle()
		.SetNormal(FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, 4.0f))
		.SetHovered(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f))
		.SetPressed(FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, 4.0f))
		.SetHoveredForeground(FStyleColors::White)
		.SetNormalForeground(FStyleColors::White)
		.SetPressedForeground(FStyleColors::White)
		.SetNormalPadding(FMargin(0.f, 4.0f))
		.SetPressedPadding(FMargin(0.f, 4.0f));
	Set("HomeScreen.ReconnectButton", ReconnectButton);

	FCheckBoxStyle MainMenuSectionCheckBox = FAppStyle::GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
	MainMenuSectionCheckBox
		.SetUncheckedHoveredImage(FSlateNoResource())
		.SetUncheckedPressedImage(FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, 4.0f))
		.SetCheckedImage(FSlateRoundedBoxBrush(FStyleColors::DropdownOutline, 4.0f))
		.SetCheckedHoveredImage(FSlateRoundedBoxBrush(FStyleColors::Hover, 4.0f));
	Set("HomeScreen.MainMenuSectionButton", MainMenuSectionCheckBox);
}

void FStarshipEditorStyle::FStyle::SetupTranslationEditorStyles()
{
	Set("TranslationEditor.Export", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_40x", Icon40x40));
	Set("TranslationEditor.PreviewInEditor", new IMAGE_BRUSH("Icons/icon_levels_visible_40x", Icon40x40));
	Set("TranslationEditor.Import", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_40x", Icon40x40));
	Set("TranslationEditor.Search", new IMAGE_BRUSH("Icons/icon_Blueprint_Find_40px", Icon40x40));
	Set("TranslationEditor.TranslationPicker", new IMAGE_BRUSH("Icons/icon_StaticMeshEd_VertColor_40x", Icon40x40));
	Set("TranslationEditor.ImportLatestFromLocalizationService", new IMAGE_BRUSH("Icons/icon_worldscript_40x", Icon40x40));
}


void FStarshipEditorStyle::FStyle::SetupLocalizationDashboardStyles()
{
	Set("LocalizationDashboard.MenuIcon", new IMAGE_BRUSH_SVG("Starship/Common/LocalizationDashboard", Icon16x16));

	Set("LocalizationDashboard.GatherTextAllTargets", new IMAGE_BRUSH("Icons/Icon_Localisation_Gather_All_40x", Icon40x40));
	Set("LocalizationDashboard.ImportTextAllTargetsAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_40x", Icon40x40));
	Set("LocalizationDashboard.ExportTextAllTargetsAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_40x", Icon40x40));
	Set("LocalizationDashboard.ImportDialogueAllTargetsAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_40x", Icon40x40));
	Set("LocalizationDashboard.ImportDialogueScriptAllTargetsAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_40x", Icon40x40));
	Set("LocalizationDashboard.ExportDialogueScriptAllTargetsAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_40x", Icon40x40));
	Set("LocalizationDashboard.CountWordsForAllTargets", new IMAGE_BRUSH("Icons/Icon_Localisation_Refresh_Word_Counts_40x", Icon40x40));
	Set("LocalizationDashboard.CompileTextAllTargetsAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Compile_Translations_40x", Icon40x40));

	Set("LocalizationDashboard.GatherTextAllTargets.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Gather_All_16x", Icon16x16));
	Set("LocalizationDashboard.ImportTextAllTargetsAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationDashboard.ExportTextAllTargetsAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_16x", Icon16x16));
	Set("LocalizationDashboard.ImportDialogueAllTargetsAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationDashboard.ImportDialogueScriptAllTargetsAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationDashboard.ExportDialogueScriptAllTargetsAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_16x", Icon16x16));
	Set("LocalizationDashboard.CountWordsForAllTargets.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Refresh_Word_Counts_16x", Icon16x16));
	Set("LocalizationDashboard.CompileTextAllTargetsAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Compile_Translations_16x", Icon16x16));

	Set("LocalizationDashboard.GatherTextTarget", new IMAGE_BRUSH("Icons/Icon_Localisation_Gather_All_16x", Icon16x16));
	Set("LocalizationDashboard.ImportTextAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationDashboard.ExportTextAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_16x", Icon16x16));
	Set("LocalizationDashboard.ImportDialogueAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationDashboard.ImportDialogueScriptAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationDashboard.ExportDialogueScriptAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_16x", Icon16x16));
	Set("LocalizationDashboard.CountWordsForTarget", new IMAGE_BRUSH("Icons/Icon_Localisation_Refresh_Word_Counts_16x", Icon16x16));
	Set("LocalizationDashboard.CompileTextAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Compile_Translations_16x", Icon16x16));
	Set("LocalizationDashboard.DeleteTarget", new IMAGE_BRUSH("Icons/Cross_12x", Icon12x12 ) );

	Set("LocalizationTargetEditor.GatherText", new IMAGE_BRUSH("Icons/Icon_Localisation_Gather_All_40x", Icon40x40));
	Set("LocalizationTargetEditor.ImportTextAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_40x", Icon40x40));
	Set("LocalizationTargetEditor.ExportTextAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_40x", Icon40x40));
	Set("LocalizationTargetEditor.ImportDialogueAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_40x", Icon40x40));
	Set("LocalizationTargetEditor.ImportDialogueScriptAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_40x", Icon40x40));
	Set("LocalizationTargetEditor.ExportDialogueScriptAllCultures", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_40x", Icon40x40));
	Set("LocalizationTargetEditor.CountWords", new IMAGE_BRUSH("Icons/Icon_Localisation_Refresh_Word_Counts_40x", Icon40x40));
	Set("LocalizationTargetEditor.CompileTextAllCultures", new IMAGE_BRUSH( "Icons/Icon_Localisation_Compile_Translations_40x", Icon40x40));

	Set("LocalizationTargetEditor.GatherText.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Gather_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.ImportTextAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.ExportTextAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.ImportDialogueAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.ImportDialogueScriptAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.ExportDialogueScriptAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.CountWords.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Refresh_Word_Counts_16x", Icon16x16));
	Set("LocalizationTargetEditor.CompileTextAllCultures.Small", new IMAGE_BRUSH("Icons/Icon_Localisation_Compile_Translations_16x", Icon16x16));

	Set("LocalizationTargetEditor.DirectoryPicker", new IMAGE_BRUSH( "Icons/ellipsis_12x", Icon12x12 ));
	Set("LocalizationTargetEditor.GatherSettingsIcon_Valid", new IMAGE_BRUSH("Settings/Settings_Good", Icon16x16));
	Set("LocalizationTargetEditor.GatherSettingsIcon_Warning", new IMAGE_BRUSH("Settings/Settings_Warning", Icon16x16));

	Set("LocalizationTargetEditor.NativeCulture", new IMAGE_BRUSH( "Icons/Star_16x", Icon16x16 ) );

	Set("LocalizationTargetEditor.EditTranslations", new IMAGE_BRUSH("Icons/icon_file_open_16px", Icon16x16));
	Set("LocalizationTargetEditor.ImportTextCulture", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.ExportTextCulture", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.ImportDialogueScriptCulture", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.ExportDialogueScriptCulture", new IMAGE_BRUSH("Icons/Icon_Localisation_Export_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.ImportDialogueCulture", new IMAGE_BRUSH("Icons/Icon_Localisation_Import_All_16x", Icon16x16));
	Set("LocalizationTargetEditor.CompileTextCulture", new IMAGE_BRUSH("Icons/Icon_Localisation_Compile_Translations_16x", Icon16x16));
	Set("LocalizationTargetEditor.DeleteCulture", new IMAGE_BRUSH("Icons/Cross_12x", Icon12x12 ) );

	Set("LocalizationTargetEditor.GatherSettings.AddMetaDataTextKeyPatternArgument", new IMAGE_BRUSH("Icons/icon_Blueprint_AddVariable_40px", Icon16x16 ) );

	Set( "LocalizationDashboard.CommandletLog.Text", FTextBlockStyle(NormalText)
		.SetFont( DEFAULT_FONT( "Regular", 8 ) )
		.SetShadowOffset( FVector2f::ZeroVector )
		);
}

void FStarshipEditorStyle::FStyle::SetupMyBlueprintStyles()
{
	Set( "MyBlueprint.DeleteEntry", new IMAGE_BRUSH("Icons/GeneralTools/Delete_40x", Icon16x16));
}

void FStarshipEditorStyle::FStyle::SetupStatusBarStyle()
{
	Set("StatusBar.ContentBrowserUp", new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-up", Icon16x16));
	Set("StatusBar.ContentBrowserDown", new CORE_IMAGE_BRUSH_SVG("Starship/Common/chevron-down", Icon16x16));

}

void FStarshipEditorStyle::FStyle::SetupColorPickerStyle()
{
	Set("ColorPicker.ColorThemes", new IMAGE_BRUSH_SVG("Starship/ColorPicker/ColorThemes", Icon16x16));
}

void FStarshipEditorStyle::FStyle::SetupSourceCodeStyles()
{
	constexpr int32 SourceCodeFontSize = 9;
	FSlateFontInfo SourceCodeFont = DEFAULT_FONT("Mono", SourceCodeFontSize);
	FTextBlockStyle NormalSourceCodeText = FTextBlockStyle(NormalText)
		.SetFont(SourceCodeFont);
	const FTextBlockStyle SourceCodeErrorText = FTextBlockStyle(NormalSourceCodeText)
		.SetUnderlineBrush(IMAGE_BRUSH("Old/White", Icon8x8, FLinearColor::Red, ESlateBrushTileType::Both))
		.SetColorAndOpacity(FLinearColor::Red);
	
	Set("SyntaxHighlight.SourceCode.Normal", FTextBlockStyle(NormalSourceCodeText).SetColorAndOpacity(FLinearColor(FColor(189, 183, 107))));
	Set("SyntaxHighlight.SourceCode.Operator", FTextBlockStyle(NormalSourceCodeText).SetColorAndOpacity(FLinearColor(FColor(220, 220, 220))));
	Set("SyntaxHighlight.SourceCode.Keyword", FTextBlockStyle(NormalSourceCodeText).SetColorAndOpacity(FLinearColor(FColor(86, 156, 214))));
	Set("SyntaxHighlight.SourceCode.String", FTextBlockStyle(NormalSourceCodeText).SetColorAndOpacity(FLinearColor(FColor(214, 157, 133))));
	Set("SyntaxHighlight.SourceCode.Number", FTextBlockStyle(NormalSourceCodeText).SetColorAndOpacity(FLinearColor(FColor(181, 206, 168))));
	Set("SyntaxHighlight.SourceCode.Comment", FTextBlockStyle(NormalSourceCodeText).SetColorAndOpacity(FLinearColor(FColor(87, 166, 74))));
	Set("SyntaxHighlight.SourceCode.PreProcessorKeyword", FTextBlockStyle(NormalSourceCodeText).SetColorAndOpacity(FLinearColor(FColor(188, 98, 171))));

	Set("SyntaxHighlight.SourceCode.Error", SourceCodeErrorText); 
}

#undef LOCTEXT_NAMESPACE

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
