// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/UObjectGlobals.h"
#include "Modules/ModuleManager.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateTypes.h"
#include "EditorStyleSet.h"
#include "Settings/EditorStyleSettings.h"
#include "ISettingsModule.h"

#if WITH_EDITOR
#include "PropertyEditorModule.h"
#endif


struct FPropertyChangedEvent;

/**
 * Declares the Editor's visual style.
 */
class FStarshipEditorStyle
	: public FEditorStyle
{
public:

	static void Initialize();
	static void Shutdown();

	static void SyncCustomizations()
	{
		FStarshipEditorStyle::StyleInstance->SyncSettings();
	}

	static const FName& GetStyleSetName();

	class FStyle : public FSlateStyleSet
	{
	public:
		FStyle();
		~FStyle();

		void Initialize();
		void SetupGeneralStyles();
		void SetupLevelGeneralStyles();
		void SetupWorldBrowserStyles();
		void SetupWorldPartitionStyles();
		void SetupSequencerStyles();
		void SetupViewportStyles();
		void SetupNewViewportToolbarStyles();
		void SetupOldViewportToolbarStyles();
		void SetupLegacyViewportToolbarStyles();
		void SetupDepthBarStyles();
		void SetupMenuBarStyles();
		void SetupGeneralIcons();
		void SetupWindowStyles();
		void SetupProjectBadgeStyle();
		void SetupDockingStyles();
		void SetupTutorialStyles();
		void SetupTranslationEditorStyles();
		void SetupLocalizationDashboardStyles();
		void SetupPropertyEditorStyles();
		void SetupProfilerStyle();
		void SetupGraphEditorStyles();
		void SetupLevelEditorStyle();
		void SetupPersonaStyle();
		void SetupClassThumbnailOverlays();
		void SetupClassIconsAndThumbnails();
		void SetupContentBrowserStyle();
		void SetupLandscapeEditorStyle();
		void SetupToolkitStyles();
		void SetupUnsavedAssetsStyles();
		void SetupSourceControlStyles();
		void SetupAutomationStyles();
		void SetupUMGEditorStyles();
		void SetupHomeScreenStyles();
		void SetupMyBlueprintStyles();
		void SetupStatusBarStyle();
		void SetupColorPickerStyle();
		void SetupDerivedDataStyle();
		void SetupSourceCodeStyles();

		void SettingsChanged(FName PropertyName);
		void SyncSettings();
		void SyncParentStyles();

		static void SetColor(const TSharedRef<FLinearColor>& Source, const FLinearColor& Value);
		static bool IncludeEditorSpecificStyles();

		static const FVector2f Icon7x16;
		static const FVector2f Icon8x4;
		static const FVector2f Icon16x4;
		static const FVector2f Icon8x8;
		static const FVector2f Icon10x10;
		static const FVector2f Icon12x12;
		static const FVector2f Icon12x16;
		static const FVector2f Icon14x14;
		static const FVector2f Icon16x16;
		static const FVector2f Icon16x20;
		static const FVector2f Icon20x20;
		static const FVector2f Icon22x22;
		static const FVector2f Icon24x24;
		static const FVector2f Icon25x25;
		static const FVector2f Icon32x32;
		static const FVector2f Icon40x40;
		static const FVector2f Icon48x48;
		static const FVector2f Icon64x64;
		static const FVector2f Icon36x24;
		static const FVector2f Icon128x128;

		// These are the colors that are updated by the user style customizations
		const TSharedRef< FLinearColor > SelectionColor_Subdued_LinearRef;
		const TSharedRef< FLinearColor > HighlightColor_LinearRef;
		const TSharedRef< FLinearColor > WindowHighlightColor_LinearRef;

		// These are the Slate colors which reference those above; these are the colors to put into the style
		// Most of these are owned by our parent style
		FSlateColor DefaultForeground;
		FSlateColor InvertedForeground;
		FSlateColor SelectorColor;
		FSlateColor SelectionColor;
		FSlateColor SelectionColor_Inactive;
		FSlateColor SelectionColor_Pressed;

		const FSlateColor SelectionColor_Subdued;
		const FSlateColor HighlightColor;
		const FSlateColor WindowHighlightColor;

		// These are common colors used throughout the editor in multiple style elements
		const FSlateColor InheritedFromBlueprintTextColor;

		// Styles inherited from the parent style
		FTextBlockStyle NormalText;
		FEditableTextBoxStyle NormalEditableTextBoxStyle;
		FTableRowStyle NormalTableRowStyle;
		FButtonStyle Button;
		FButtonStyle HoverHintOnly;
		FButtonStyle NoBorder;
		FScrollBarStyle ScrollBar;
		FSlateFontInfo NormalFont;

		FSlateBrush* WindowTitleOverride;

		FDelegateHandle SettingChangedHandler;
	};

	static TSharedRef<class FStarshipEditorStyle::FStyle> Create()
	{
		TSharedRef<class FStarshipEditorStyle::FStyle> NewStyle = MakeShareable(new FStarshipEditorStyle::FStyle());
		NewStyle->Initialize();

		return NewStyle;
	}

	static TSharedPtr<FStarshipEditorStyle::FStyle> StyleInstance;
	static FName StyleSetName;
};
