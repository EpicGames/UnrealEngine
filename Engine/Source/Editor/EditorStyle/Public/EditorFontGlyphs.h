// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API EDITORSTYLE_API

class FText;

class FEditorFontGlyphs
{
public:
	// (laquo, quote, previous, back) http://fortawesome.github.io/Font-Awesome/icon/angle-double-left/
	static UE_API FText Angle_Double_Left;

	// (raquo, quote, next, forward) http://fortawesome.github.io/Font-Awesome/icon/angle-double-right/
	static UE_API FText Angle_Double_Right;

	// (previous, back) http://fortawesome.github.io/Font-Awesome/icon/angle-left/
	static UE_API FText Angle_Left;

	// (next, forward) http://fortawesome.github.io/Font-Awesome/icon/angle-right/
	static UE_API FText Angle_Right;

	// http://fortawesome.github.io/Font-Awesome/icon/angle-up/
	static UE_API FText Angle_Up;

	// (box, storage) http://fortawesome.github.io/Font-Awesome/icon/archive/
	static UE_API FText Archive;

	// (next, forward) http://fortawesome.github.io/Font-Awesome/icon/arrow-circle-o-right/
	static UE_API FText Arrow_Circle_O_Right;

	// (download) http://fortawesome.github.io/Font-Awesome/icon/arrow-down/
	static UE_API FText Arrow_Down;

	// (previous, back) http://fortawesome.github.io/Font-Awesome/icon/arrow-left/
	static UE_API FText Arrow_Left;

	// (next, forward) http://fortawesome.github.io/Font-Awesome/icon/arrow-right/
	static UE_API FText Arrow_Right;

	// (move, reorder, resize) http://fortawesome.github.io/Font-Awesome/icon/arrows/
	static UE_API FText Arrows;

	// (delete, remove, trash, hide, block, stop, abort, cancel) http://fortawesome.github.io/Font-Awesome/icon/ban/
	static UE_API FText Ban;

	// (menu, drag, reorder, settings, list, ul, ol, checklist, todo, list, hamburger) http://fortawesome.github.io/Font-Awesome/icon/bars/
	static UE_API FText Bars;

	// (read, documentation) http://fortawesome.github.io/Font-Awesome/icon/book/
	static UE_API FText Book;

	// (report, insect) http://fortawesome.github.io/Font-Awesome/icon/bug/
	static UE_API FText Bug;

	// (vehicle) http://fortawesome.github.io/Font-Awesome/icon/car/
	static UE_API FText Car;

	// (more, dropdown, menu, triangle down) http://fortawesome.github.io/Font-Awesome/icon/caret-down/
	static UE_API FText Caret_Down;

	// (next, forward, triangle right) http://fortawesome.github.io/Font-Awesome/icon/caret-right/
	static UE_API FText Caret_Right;

	// (more, dropdown, menu) http://fortawesome.github.io/Font-Awesome/icon/caret-square-o-down/
	static UE_API FText Caret_Square_O_Down;

	// (previous, back) http://fortawesome.github.io/Font-Awesome/icon/caret-square-o-left/
	static UE_API FText Caret_Square_O_Left;

	// (next, forward) http://fortawesome.github.io/Font-Awesome/icon/caret-square-o-right/
	static UE_API FText Caret_Square_O_Right;

	// http://fortawesome.github.io/Font-Awesome/icon/caret-square-o-up/
	static UE_API FText Caret_Square_O_Up;

	// (triangle up) http://fortawesome.github.io/Font-Awesome/icon/caret-up/
	static UE_API FText Caret_Up;

	// (checkmark, done, todo, agree, accept, confirm, tick) http://fortawesome.github.io/Font-Awesome/icon/check/
	static UE_API FText Check;

	// (todo, done, agree, accept, confirm) http://fortawesome.github.io/Font-Awesome/icon/check-circle/
	static UE_API FText Check_Circle;

	// (dot, notification) http://fortawesome.github.io/Font-Awesome/icon/circle/
	static UE_API FText Circle;

	// http://fortawesome.github.io/Font-Awesome/icon/circle-o/
	static UE_API FText Circle_O;

	// (watch, timer, late, timestamp) http://fortawesome.github.io/Font-Awesome/icon/clock-o/
	static UE_API FText Clock_O;

	// (settings) http://fortawesome.github.io/Font-Awesome/icon/cog/
	static UE_API FText Cog;

	// (settings) http://fortawesome.github.io/Font-Awesome/icon/cogs/
	static UE_API FText Cogs;

	// (picker) http://fortawesome.github.io/Font-Awesome/icon/crosshairs/
	static UE_API FText Crosshairs;

	// http://fortawesome.github.io/Font-Awesome/icon/database/
	static UE_API FText Database;

	// (import) http://fortawesome.github.io/Font-Awesome/icon/download/
	static UE_API FText Download;

	// http://fortawesome.github.io/Font-Awesome/icon/eraser/
	static UE_API FText Eraser;

	// (transfer, arrows) http://fortawesome.github.io/Font-Awesome/icon/exchange/
	static UE_API FText Exchange;

	// (warning, error, problem, notification, notify, alert) http://fortawesome.github.io/Font-Awesome/icon/exclamation/
	static UE_API FText Exclamation;

	// (warning, error, problem, notification, alert) http://fortawesome.github.io/Font-Awesome/icon/exclamation-triangle/
	static UE_API FText Exclamation_Triangle;

	// (enlarge, bigger, resize) http://fortawesome.github.io/Font-Awesome/icon/expand/
	static UE_API FText Expand;

	// (open, new) http://fortawesome.github.io/Font-Awesome/icon/external-link/
	static UE_API FText External_Link;

	// (show, visible, views) http://fortawesome.github.io/Font-Awesome/icon/eye/
	static UE_API FText Eye;

	// (toggle, show, hide, visible, visiblity, views) http://fortawesome.github.io/Font-Awesome/icon/eye-slash/
	static UE_API FText Eye_Slash;

	// (next, end, last) http://fortawesome.github.io/Font-Awesome/icon/fast-forward/
	static UE_API FText Fast_Forward;

	// (new, page, pdf, document) http://fortawesome.github.io/Font-Awesome/icon/file/
	static UE_API FText File;

	// (movie) http://fortawesome.github.io/Font-Awesome/icon/film/
	static UE_API FText Film;

	// (funnel, options) http://fortawesome.github.io/Font-Awesome/icon/filter/
	static UE_API FText Filter;

	// http://fortawesome.github.io/Font-Awesome/icon/floppy-o/
	static UE_API FText Floppy_O;

	// http://fortawesome.github.io/Font-Awesome/icon/folder/
	static UE_API FText Folder;

	// http://fortawesome.github.io/Font-Awesome/icon/folder-open/
	static UE_API FText Folder_Open;

	// http://fortawesome.github.io/Font-Awesome/icon/hourglass/
	static UE_API FText Hourglass;

	// http://fortawesome.github.io/Font-Awesome/icon/hourglass-o/
	static UE_API FText Hourglass_O;

	// (help, information, more, details) http://fortawesome.github.io/Font-Awesome/icon/info/
	static UE_API FText Info;

	// (help, information, more, details) http://fortawesome.github.io/Font-Awesome/icon/info-circle/
	static UE_API FText Info_Circle;

	// http://fortawesome.github.io/Font-Awesome/icon/level-down/
	static UE_API FText Level_Down;

	// http://fortawesome.github.io/Font-Awesome/icon/level-up/
	static UE_API FText Level_Up;

	// (idea, inspiration) http://fortawesome.github.io/Font-Awesome/icon/lightbulb-o/
	static UE_API FText Lightbulb_O;

	// (graph, analytics) http://fortawesome.github.io/Font-Awesome/icon/line-chart/
	static UE_API FText Line_Chart;

	// (chain) http://fortawesome.github.io/Font-Awesome/icon/link/
	static UE_API FText Link;

	// (protect, admin) http://fortawesome.github.io/Font-Awesome/icon/lock/
	static UE_API FText Lock;

	// http://fortawesome.github.io/Font-Awesome/icon/long-arrow-down/
	static UE_API FText Long_Arrow_Down;

	// http://fortawesome.github.io/Font-Awesome/icon/long-arrow-right/
	static UE_API FText Long_Arrow_Right;

	// http://fortawesome.github.io/Font-Awesome/icon/map/
	static UE_API FText Map;

	// (map, pin, location, coordinates, localize, address, travel, where, place) http://fortawesome.github.io/Font-Awesome/icon/map-marker/
	static UE_API FText Map_Marker;

	// (hide, minify, delete, remove, trash, hide, collapse) http://fortawesome.github.io/Font-Awesome/icon/minus/
	static UE_API FText Minus;

	// (delete, remove, trash, hide) http://fortawesome.github.io/Font-Awesome/icon/minus-circle/
	static UE_API FText Minus_Circle;

	// http://fortawesome.github.io/Font-Awesome/icon/paper-plane/
	static UE_API FText Paper_Plane;

	// (wait) http://fortawesome.github.io/Font-Awesome/icon/pause/
	static UE_API FText Pause;

	// http://fortawesome.github.io/Font-Awesome/icon/pause-circle/
	static UE_API FText Pause_Circle;

	// (write, edit, update) http://fortawesome.github.io/Font-Awesome/icon/pencil/
	static UE_API FText Pencil;

	// (write, edit, update) http://fortawesome.github.io/Font-Awesome/icon/pencil-square/
	static UE_API FText Pencil_Square;

	// (start, playing, music, sound) http://fortawesome.github.io/Font-Awesome/icon/play/
	static UE_API FText Play;

	// (start, playing) http://fortawesome.github.io/Font-Awesome/icon/play-circle/
	static UE_API FText Play_Circle;

	// http://fortawesome.github.io/Font-Awesome/icon/plug/
	static UE_API FText Plug;

	// (add, new, create, expand) http://fortawesome.github.io/Font-Awesome/icon/plus/
	static UE_API FText Plus;

	// (add, new, create, expand) http://fortawesome.github.io/Font-Awesome/icon/plus-circle/
	static UE_API FText Plus_Circle;

	// (help, information, unknown, support) http://fortawesome.github.io/Font-Awesome/icon/question/
	static UE_API FText Question;

	// (sort, shuffle) http://fortawesome.github.io/Font-Awesome/icon/random/
	static UE_API FText Random;

	// http://fortawesome.github.io/Font-Awesome/icon/recycle/
	static UE_API FText Recycle;

	// (reload, sync) http://fortawesome.github.io/Font-Awesome/icon/refresh/
	static UE_API FText Refresh;

	// http://fortawesome.github.io/Font-Awesome/icon/share/
	static UE_API FText Share;

	// http://fortawesome.github.io/Font-Awesome/icon/share-alt/
	static UE_API FText Share_Alt;

	// (enter, join, log in, login, sign up, sign in, signin, signup, arrow) http://fortawesome.github.io/Font-Awesome/icon/sign-in/
	static UE_API FText Sign_In;

	// (log out, logout, leave, exit, arrow) http://fortawesome.github.io/Font-Awesome/icon/sign-out/
	static UE_API FText Sign_Out;

	// (block, box) http://fortawesome.github.io/Font-Awesome/icon/square/
	static UE_API FText Square;

	// (award, achievement, night, rating, score) http://fortawesome.github.io/Font-Awesome/icon/star/
	static UE_API FText Star;

	// (next, end, last) http://fortawesome.github.io/Font-Awesome/icon/step-forward/
	static UE_API FText Step_Forward;

	// (block, box, square) http://fortawesome.github.io/Font-Awesome/icon/stop/
	static UE_API FText Stop;

	// (command, prompt, code) http://fortawesome.github.io/Font-Awesome/icon/terminal/
	static UE_API FText Terminal;

	// (blocks, squares, boxes) http://fortawesome.github.io/Font-Awesome/icon/th/
	static UE_API FText Th;

	// (marker, pin, location, coordinates) http://fortawesome.github.io/Font-Awesome/icon/thumb-tack/
	static UE_API FText Thumb_Tack;

	// (close, exit, x, cross) http://fortawesome.github.io/Font-Awesome/icon/times/
	static UE_API FText Times;

	// (close, exit, x) http://fortawesome.github.io/Font-Awesome/icon/times-circle/
	static UE_API FText Times_Circle;

	// (garbage, delete, remove, hide) http://fortawesome.github.io/Font-Awesome/icon/trash/
	static UE_API FText Trash;

	// (back) http://fortawesome.github.io/Font-Awesome/icon/undo/
	static UE_API FText Undo;

	// (protect, admin, password, lock) http://fortawesome.github.io/Font-Awesome/icon/unlock/
	static UE_API FText Unlock;

	// (film, movie, record) http://fortawesome.github.io/Font-Awesome/icon/video-camera/
	static UE_API FText Video_Camera;
};

#undef UE_API
