// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Dependencies
 *****************************************************************************/

#include "CoreUObject.h"
#include "Modules/ModuleInterface.h"
#include "SlateBasics.h"


/* Interfaces
 *****************************************************************************/

#include "Interfaces/IEditorStyleModule.h"
#include "Styling/AppStyle.h"


/* Classes
 *****************************************************************************/

#include "EditorFontGlyphs.h"
