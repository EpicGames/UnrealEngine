// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Brushes/SlateDynamicImageBrush.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Fonts/SlateFontInfo.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMath.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Sound/SlateSound.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateColor.h"
#include "Styling/StyleDefaults.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API EDITORSTYLE_API

struct FLinearColor;
struct FMargin;
struct FSlateBrush;
struct FSlateDynamicImageBrush;
struct FSlateSound;

/**
 * A collection of named properties that guide the appearance of Slate.
 */
class FEditorStyle
{
public:

	/** 
	* @return the Application Style 
	*
	* NOTE: Until the Editor can be fully updated, calling FEditorStyle::Get() or any of its 
	* static convenience functions will will return the AppStyle instead of the style definied in this class.  
	*
	* Using the AppStyle is preferred in most cases as it allows the style to be changed 
	* on an application level.
	*
	* In cases requiring explicit use of the EditorStyle where a Slate Widget should not take on
	* the appearance of the rest of the application, use FEditorStyle::GetEditorStyle().
	*
	*/
	UE_DEPRECATED(5.1, "FEditorStyle::Get() is deprecated, use FAppStyle::Get() instead.")
	static const ISlateStyle& Get( )
	{
		return FAppStyle::Get();
	}

	template< class T >      
	UE_DEPRECATED(5.1, "FEditorStyle::GetWidgetStyle() is deprecated, use FAppStyle::GetWidgetStyle() instead.")
	static const T& GetWidgetStyle( FName PropertyName, const ANSICHAR* Specifier = NULL  ) 
	{
		return FAppStyle::Get().GetWidgetStyle< T >( PropertyName, Specifier );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetFloat() is deprecated, use FAppStyle::GetFloat() instead.")
	static float GetFloat( FName PropertyName, const ANSICHAR* Specifier = NULL )
	{
		return FAppStyle::Get().GetFloat( PropertyName, Specifier );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetVector() is deprecated, use FAppStyle::GetVector() instead.")
	static FVector2D GetVector( FName PropertyName, const ANSICHAR* Specifier = NULL ) 
	{
		return FAppStyle::Get().GetVector( PropertyName, Specifier );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetColor() is deprecated, use FAppStyle::GetColor() instead.")
	static const FLinearColor& GetColor( FName PropertyName, const ANSICHAR* Specifier = NULL )
	{
		return FAppStyle::Get().GetColor( PropertyName, Specifier );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetSlateColor() is deprecated, use FAppStyle::GetSlateColor() instead.")
	static const FSlateColor GetSlateColor( FName PropertyName, const ANSICHAR* Specifier = NULL )
	{
		return FAppStyle::Get().GetSlateColor( PropertyName, Specifier );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetMargin() is deprecated, use FAppStyle::GetMargin() instead.")
	static const FMargin& GetMargin( FName PropertyName, const ANSICHAR* Specifier = NULL )
	{
		return FAppStyle::Get().GetMargin( PropertyName, Specifier );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetBrush() is deprecated, use FAppStyle::GetBrush() instead.")
	static const FSlateBrush* GetBrush( FName PropertyName, const ANSICHAR* Specifier = NULL )
	{
		return FAppStyle::Get().GetBrush( PropertyName, Specifier );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetDynamicImageBrush() is deprecated, use FAppStyle::GetDynamicImageBrush() instead.")
	static const TSharedPtr< FSlateDynamicImageBrush > GetDynamicImageBrush( FName BrushTemplate, FName TextureName, const ANSICHAR* Specifier = NULL )
	{
		return FAppStyle::Get().GetDynamicImageBrush( BrushTemplate, TextureName, Specifier );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetDynamicImageBrush() is deprecated, use FAppStyle::GetDynamicImageBrush() instead.")
	static const TSharedPtr< FSlateDynamicImageBrush > GetDynamicImageBrush( FName BrushTemplate, const ANSICHAR* Specifier, class UTexture2D* TextureResource, FName TextureName )
	{
		return FAppStyle::Get().GetDynamicImageBrush( BrushTemplate, Specifier, TextureResource, TextureName );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetDynamicImageBrush() is deprecated, use FAppStyle::GetDynamicImageBrush() instead.")
	static const TSharedPtr< FSlateDynamicImageBrush > GetDynamicImageBrush( FName BrushTemplate, class UTexture2D* TextureResource, FName TextureName )
	{
		return FAppStyle::Get().GetDynamicImageBrush( BrushTemplate, TextureResource, TextureName );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetSound() is deprecated, use FAppStyle::GetSound() instead.")
	static const FSlateSound& GetSound( FName PropertyName, const ANSICHAR* Specifier = NULL )
	{
		return FAppStyle::Get().GetSound( PropertyName, Specifier );
	}
	
	UE_DEPRECATED(5.1, "FEditorStyle::GetFontStyle() is deprecated, use FAppStyle::GetFontStyle() instead.")
	static FSlateFontInfo GetFontStyle( FName PropertyName, const ANSICHAR* Specifier = NULL )
	{
		return FAppStyle::Get().GetFontStyle( PropertyName, Specifier );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetDefaultBrush() is deprecated, use FAppStyle::GetDefaultBrush() instead.")
	static const FSlateBrush* GetDefaultBrush()
	{
		return FAppStyle::Get().GetDefaultBrush();
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetNoBrush() is deprecated, use FAppStyle::GetNoBrush() instead.")
	static const FSlateBrush* GetNoBrush()
	{
		return FStyleDefaults::GetNoBrush();
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetOptionalBrush() is deprecated, use FAppStyle::GetOptionalBrush() instead.")
	static const FSlateBrush* GetOptionalBrush( FName PropertyName, const ANSICHAR* Specifier = NULL, const FSlateBrush* const DefaultBrush = FStyleDefaults::GetNoBrush() )
	{
		return FAppStyle::Get().GetOptionalBrush( PropertyName, Specifier, DefaultBrush );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetResources() is deprecated, use FAppStyle::GetResources() instead.")
	static void GetResources( TArray< const FSlateBrush* >& OutResources )
	{
		return FAppStyle::Get().GetResources( OutResources );
	}

	UE_DEPRECATED(5.1, "FEditorStyle::GetStyleSetName() is deprecated, use FAppStyle::GetAppStyleSetName() instead.")
	static const FName& GetStyleSetName()
	{
		return Instance->GetStyleSetName();
	}

	/**
	 * Concatenates two FNames.e If A and B are "Path.To" and ".Something"
	 * the result "Path.To.Something".
	 *
	 * @param A  First FName
	 * @param B  Second name
	 *
	 * @return New FName that is A concatenated with B.
	 */
	UE_DEPRECATED(5.1, "FEditorStyle::Join() is deprecated, use FAppStyle::Join() instead.")
	static FName Join( FName A, const ANSICHAR* B )
	{
		if( B == NULL )
		{
			return A;
		}
		else
		{
			return FName( *( A.ToString() + B ) );
		}
	}

	static UE_API void ResetToDefault();


protected:

	static UE_API void SetStyle( const TSharedRef< class ISlateStyle >& NewStyle );

private:

	/** Singleton instance of the slate style */
	static UE_API TSharedPtr< class ISlateStyle > Instance;
};

#undef UE_API
